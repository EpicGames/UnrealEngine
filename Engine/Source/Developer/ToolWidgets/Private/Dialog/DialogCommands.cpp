// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dialog/DialogCommands.h"

#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "CustomDialogCommands"

UE_DEFINE_TCOMMANDS(FDialogCommands)

FDialogCommands::FDialogCommands()
	: TCommands<FDialogCommands>
	(
		TEXT("Dialogs"), // Context name for fast lookup
		NSLOCTEXT("Contexts", "DialogCommands", "Dialog Commands"), // Localized context name for displaying
		NAME_None,
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
{
}

void FDialogCommands::RegisterCommands()
{
	UI_COMMAND(Cancel, "Cancel", "Cancel the current dialog", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));
	UI_COMMAND(Confirm, "Confirm", "Confirm the current dialog", EUserInterfaceActionType::Button, FInputChord(EKeys::Enter));
}

#undef LOCTEXT_NAMESPACE