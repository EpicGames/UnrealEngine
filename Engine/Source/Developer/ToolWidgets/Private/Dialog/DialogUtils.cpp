// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dialog/DialogUtils.h"

#include "GenericPlatform/GenericPlatformMisc.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"

const FDialogButtonTexts& FDialogButtonTexts::Get()
{
	static FDialogButtonTexts ButtonTexts;
	return ButtonTexts;
}

FDialogButtonTexts::FDialogButtonTexts()
	: No(NSLOCTEXT("Dialogs", "EAppReturnTypeNo", "No"))
	, Yes(NSLOCTEXT("Dialogs", "EAppReturnTypeYes", "Yes"))
	, YesAll(NSLOCTEXT("Dialogs", "EAppReturnTypeYesAll", "Yes All"))
	, NoAll(NSLOCTEXT("Dialogs", "EAppReturnTypeNoAll", "No All"))
	, Cancel(NSLOCTEXT("Dialogs", "EAppReturnTypeCancel", "Cancel"))
	, Ok(NSLOCTEXT("Dialogs", "EAppReturnTypeOk", "OK"))
	, Retry(NSLOCTEXT("Dialogs", "EAppReturnTypeRetry", "Retry"))
	, Continue(NSLOCTEXT("Dialogs", "EAppReturnTypeContinue", "Continue"))
{
}

const FSlateBrush* FDialogUtils::GetMessageCategoryIcon(const EAppMsgCategory MessageCategory)
{
	FName MessageCategoryBrushName;

	switch (MessageCategory)
	{
		case EAppMsgCategory::Info:
		{
			MessageCategoryBrushName = "Icons.InfoWithColor.Large";
			break;
		}
		case EAppMsgCategory::Warning:
		{
			MessageCategoryBrushName = "Icons.WarningWithColor.Large";
			break;
		}
		case EAppMsgCategory::Error:
		{
			MessageCategoryBrushName = "Icons.ErrorWithColor.Large";
			break;
		}
		case EAppMsgCategory::Success:
		{
			MessageCategoryBrushName = "Icons.SuccessWithColor.Large";
			break;
		}
	}

	return !MessageCategoryBrushName.IsNone() ? FAppStyle::Get().GetBrush(MessageCategoryBrushName) : nullptr;
}
