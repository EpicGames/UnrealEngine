// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dialog/SCustomDialog.h"

#include "CoreGlobals.h"
#include "Dialog/DialogCommands.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Misc/App.h"
#include "Misc/AutomationTest.h"
#include "Misc/FeedbackContext.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"

SCustomDialog::FArguments& SCustomDialog::FArguments::IconBrush(FName InIconBrush)
{
	const FSlateBrush* ImageBrush = FAppStyle::Get().GetBrush(InIconBrush);
	if (ensureMsgf(ImageBrush != nullptr, TEXT("Brush %s is unknown"), *InIconBrush.ToString()))
	{
		_Icon = ImageBrush;
	}
	return *this;
}

void SCustomDialog::Construct(const FArguments& InArgs)
{
	OnClosed = InArgs._OnClosed;
	bAutoCloseOnButtonPress = InArgs._AutoCloseOnButtonPress;
	OnCancelHotkeyPressed = InArgs._OnCancelHotkeyPressed;
	OnConfirmHotkeyPressed = InArgs._OnConfirmHotkeyPressed;
	GetWidgetToFocusOnActivate = InArgs._GetWidgetToFocusOnActivate;

	// Build the command list, adding the Cancel and Confirm commands from FDialogCommands
	CommandList = MakeShareable(new FUICommandList);
	CommandList->MapAction(FDialogCommands::Get().Cancel,
		FUIAction(
			FSimpleDelegate::CreateSP(this, &SCustomDialog::ExecuteCancel),
			FCanExecuteAction::CreateSP(this, &SCustomDialog::CanExecuteCancel)
		)
	);
	CommandList->MapAction(FDialogCommands::Get().Confirm,
		FUIAction(
			FSimpleDelegate::CreateSP(this, &SCustomDialog::ExecuteConfirm),
			FCanExecuteAction::CreateSP(this, &SCustomDialog::CanExecuteConfirm)
		)
	);

	TSharedPtr<SVerticalBox> VerticalBox;
	SWindow::Construct( SWindow::FArguments(InArgs._WindowArguments)
		.Title(InArgs._Title)
		.ClientSize(InArgs._ClientSize) // If the ClientSize is left at zero, use Autosized
		.SizingRule(InArgs._ClientSize.IsNearlyZero() ? ESizingRule::Autosized : ESizingRule::UserSized)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		[
			SNew(SBorder)
			.Padding(InArgs._RootPadding)
			.BorderImage(FAppStyle::Get().GetBrush( "ToolPanel.GroupBorder" ))
			[
				SAssignNew(VerticalBox, SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					CreateContentBox(InArgs)
				]
			]
		] );

	// Only create the button box if there are buttons to generate
	if (InArgs._Buttons.Num() > 0)
	{
		VerticalBox->AddSlot()
			.HAlign(InArgs._HAlignButtonBox)
			.AutoHeight()
			.Padding(InArgs._ButtonAreaPadding)
			[
				CreateButtonBox(InArgs)
			];
	}
}

int32 SCustomDialog::ShowModal(const int32 InDefaultReturnValue)
{
	int32 Result = InDefaultReturnValue;
	if (!FApp::IsUnattended() && !GIsRunningUnattendedScript && FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().AddModalWindow(StaticCastSharedRef<SWindow>(this->AsShared()), FGlobalTabmanager::Get()->GetRootWindow());
		if (LastPressedButton != INDEX_NONE)
		{
			Result = LastPressedButton;
		}
	}
	else
	{
		OnClosed.ExecuteIfBound();
		if (GWarn)
		{
			GWarn->Logf(TEXT("Message dialog was triggered in unattended script mode, title: %s"), *Title.Get().ToString());
		}
	}

	if (GWarn)
	{
		GWarn->Logf(TEXT("Message dialog closed, result: %d, title: %s"), Result, *Title.Get().ToString());
	}

	return Result;
}

void SCustomDialog::SetOnCancelHotkeyPressed(const FSimpleDelegate& InOnCancelHotkeyPressed)
{
	// Don't use in conjunction with an auto-generated Cancel button
	ensureMsgf(!bGeneratedCancelButton, TEXT("Cannot use both a Cancel function and auto-generated button with the Cancel role"));

	OnCancelHotkeyPressed = InOnCancelHotkeyPressed;
}

void SCustomDialog::SetOnConfirmHotkeyPressed(const FSimpleDelegate& InOnConfirmHotkeyPressed)
{
	// Don't use in conjunction with an auto-generated Confirm button
	ensureMsgf(!bGeneratedConfirmButton, TEXT("Cannot use both a Confirm function and auto-generated button with the Confirm role"));

	OnConfirmHotkeyPressed = InOnConfirmHotkeyPressed;
}

void SCustomDialog::SetGetWidgetToFocusOnActivate(const FGetFocusWidget& InGetWidgetToFocusOnActivate)
{
	GetWidgetToFocusOnActivate = InGetWidgetToFocusOnActivate;
}

FReply SCustomDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return SWindow::OnKeyDown(MyGeometry, InKeyEvent);
}

bool SCustomDialog::OnIsActiveChanged(const FWindowActivateEvent& ActivateEvent)
{
	// If there's a function to get the focus on activation, use that to set the widget to focus
	if (GetWidgetToFocusOnActivate.IsBound())
	{
		WidgetToFocusOnActivate = GetWidgetToFocusOnActivate.Execute();
	}

	FWindowActivateEvent ModifiedActivateEvent{ ActivateEvent };

	// If this is a mouse activation, switch to keyboard activation so focus-on-activate logic won't be skipped
	if (ActivateEvent.GetActivationType() == FWindowActivateEvent::EA_ActivateByMouse)
	{
		ModifiedActivateEvent.SetActivationType(FWindowActivateEvent::EA_Activate);
	}

	return SWindow::OnIsActiveChanged(ModifiedActivateEvent);
}

void SCustomDialog::Show()
{
	if (!FApp::IsUnattended() && !GIsRunningUnattendedScript && FSlateApplication::IsInitialized())
	{
		const TSharedRef<SWindow> Window = FSlateApplication::Get().AddWindow(StaticCastSharedRef<SWindow>(this->AsShared()), true);
		if (OnClosed.IsBound())
		{
			Window->GetOnWindowClosedEvent().AddSPLambda(this, [this](const TSharedRef<SWindow>&) { OnClosed.Execute(); });
		}
	}
	else
	{
		OnClosed.ExecuteIfBound();
		if (GWarn)
		{
			GWarn->Logf(TEXT("Message dialog was triggered in unattended script mode, title: %s"), *Title.Get().ToString());
		}
	}

	if (GWarn)
	{
		GWarn->Logf(TEXT("Message dialog closed, title: %s"), *Title.Get().ToString());
	}
}

TSharedRef<SWidget> SCustomDialog::CreateContentBox(const FArguments& InArgs)
{
	TSharedRef<SHorizontalBox> ContentBox = SNew(SHorizontalBox);

	ContentBox->AddSlot()
		.AutoWidth()
		.VAlign(InArgs._VAlignIcon)
		.HAlign(InArgs._HAlignIcon)
		[
			SNew(SBox)
			.Padding(0.f, 0.f, 8.f, 0.f)
			.Visibility_Lambda([IconAttr = InArgs._Icon]() { return IconAttr.Get(nullptr) ? EVisibility::Visible : EVisibility::Collapsed; })
			[
				SNew(SImage)
				.DesiredSizeOverride(InArgs._IconDesiredSizeOverride)
				.Image(InArgs._Icon)
			]
		];

	if (InArgs._UseScrollBox)
	{
		ContentBox->AddSlot()
		.VAlign(InArgs._VAlignContent)
		.HAlign(InArgs._HAlignContent)
		.Padding(InArgs._ContentAreaPadding)
		[
			SNew(SBox)
			.MaxDesiredHeight(InArgs._ScrollBoxMaxHeight)
			[
				SNew(SScrollBox)
				+ SScrollBox::Slot()
				[
					InArgs._Content.Widget
				]
			]
		];
	}
	else
	{
		ContentBox->AddSlot()
			.FillWidth(1.0f)
			.VAlign(InArgs._VAlignContent)
			.HAlign(InArgs._HAlignContent)
			.Padding(InArgs._ContentAreaPadding)
			[
				InArgs._Content.Widget
			];
	}
	
	return ContentBox;
}

TSharedRef<SWidget> SCustomDialog::CreateButtonBox(const FArguments& InArgs)
{
	TSharedPtr<SHorizontalBox> ButtonPanel;
	TSharedRef<SHorizontalBox> ButtonBox =
		SNew(SHorizontalBox)
	
		// Before buttons
		+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				InArgs._BeforeButtons.Widget
			]

		// Buttons
		+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				SAssignNew(ButtonPanel, SHorizontalBox)
			];

	bool bCanFocusLastPrimary = true;
	bool bFocusedAnyButtonYet = false;
	for (int32 ButtonIndex = 0; ButtonIndex < InArgs._Buttons.Num(); ++ButtonIndex)
	{
		const FButton& Button = InArgs._Buttons[ButtonIndex];

		const FButtonStyle* ButtonStyle = Button.bIsPrimary ?
				&FAppStyle::Get().GetWidgetStyle< FButtonStyle >( "PrimaryButton" ) :
				&FAppStyle::Get().GetWidgetStyle< FButtonStyle >("Button");

		TSharedRef<SButton> ButtonWidget = SNew(SButton)
			.IsEnabled(Button.ButtonIsEnabled)
			.ToolTipText(Button.ButtonToolTipText)
			.OnClicked(FOnClicked::CreateSP(
				this, &SCustomDialog::OnButtonClicked, Button.OnClicked, ButtonIndex))
			.ButtonStyle(ButtonStyle)
			[
				SNew(STextBlock)
				.Text(Button.ButtonText)
			];

		ButtonPanel->AddSlot()
			.Padding(FAppStyle::Get().GetMargin("StandardDialog.SlotPadding"))
			.AutoWidth()
		[
			ButtonWidget
		];

		if (Button.bShouldFocus)
		{
			bCanFocusLastPrimary = false;
		}
		
		const bool bIsLastButton = ButtonIndex == InArgs._Buttons.Num() - 1;
		if (Button.bShouldFocus
			|| (bCanFocusLastPrimary && (Button.bIsPrimary || Button.ButtonRole == EButtonRole::Confirm))
			|| (bIsLastButton && !bFocusedAnyButtonYet))
		{
			bFocusedAnyButtonYet = true;
			SetWidgetToFocusOnActivate(ButtonWidget);
		}

		// Remember the buttons for later, so we can trigger them via hotkeys
		if (Button.ButtonRole == EButtonRole::Cancel)
		{
			// There should only be one auto-generated button with the Cancel role or a Cancel function
			ensureMsgf(!bGeneratedCancelButton && !OnCancelHotkeyPressed.IsBound(), TEXT("Only use one Cancel function or Cancel role button."));

			bGeneratedCancelButton = true;
			OnCancelHotkeyPressed = FSimpleDelegate::CreateSP(
				this, &SCustomDialog::OnHotkeyPressed, Button.OnClicked, ButtonIndex);
		}
		else if (Button.ButtonRole == EButtonRole::Confirm)
		{
			// There should only be one auto-generated button with the Confirm role or a Confirm function
			ensureMsgf(!bGeneratedConfirmButton && !OnConfirmHotkeyPressed.IsBound(), TEXT("Only use one Confirm function or Confirm role button."));

			bGeneratedConfirmButton = true;
			OnConfirmHotkeyPressed = FSimpleDelegate::CreateSP(
				this, &SCustomDialog::OnHotkeyPressed, Button.OnClicked, ButtonIndex);
		}
	}
	return ButtonBox;
}

/** Handle the button being clicked */
FReply SCustomDialog::OnButtonClicked(FSimpleDelegate OnClicked, int32 ButtonIndex)
{
	OnHotkeyPressed(OnClicked, ButtonIndex);
	return FReply::Handled();
}

/** Handle the hotkey being pressed */
void SCustomDialog::OnHotkeyPressed(FSimpleDelegate OnClicked, int32 ButtonIndex)
{
	LastPressedButton = ButtonIndex;

	if (bAutoCloseOnButtonPress)
	{
		RequestDestroyWindow();
	}

	OnClicked.ExecuteIfBound();
}

bool SCustomDialog::CanExecuteCancel() const
{
	// Return true if Cancel button function is available
	return OnCancelHotkeyPressed.IsBound();
}

void SCustomDialog::ExecuteCancel()
{
	// Try using the Cancel function
	OnCancelHotkeyPressed.ExecuteIfBound();
}

bool SCustomDialog::CanExecuteConfirm() const
{
	// Return true if a Confirm button function is available
	return OnConfirmHotkeyPressed.IsBound();
}

void SCustomDialog::ExecuteConfirm()
{
	// Try using the Confirm function
	OnConfirmHotkeyPressed.ExecuteIfBound();
}

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(SCustomDialogShowModalTest, "System.ToolWidgets.CustomDialog.ShowModalInvokesOnClosedWhenUnattended",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::CommandletContext | EAutomationTestFlags::EngineFilter)
 
bool SCustomDialogShowModalTest::RunTest(const FString& Parameters)
{
	bool bClosed = false;
	const TSharedRef<SCustomDialog> Dialog = SNew(SCustomDialog)
		.Title(FText::FromString("Test dialog."))
		.OnClosed_Lambda([&bClosed] {	bClosed = true;	});
	
	Dialog->ShowModal();
		
	return bClosed == true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(SCustomDialogShowTest, "System.ToolWidgets.CustomDialog.ShowInvokesOnClosedWhenUnattended",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::CommandletContext | EAutomationTestFlags::EngineFilter)
 
bool SCustomDialogShowTest::RunTest(const FString& Parameters)
{
	bool bClosed = false;
	const TSharedRef<SCustomDialog> Dialog = SNew(SCustomDialog)
		.Title(FText::FromString("Test dialog."))
		.OnClosed_Lambda([&bClosed] {	bClosed = true;	});
	
	Dialog->Show();
		
	return bClosed == true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(SCustomDialogShowModalReturnTest, "System.ToolWidgets.CustomDialog.ShowModalReturnsDefaultValueInUnattendedMode",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::CommandletContext | EAutomationTestFlags::EngineFilter)
 
bool SCustomDialogShowModalReturnTest::RunTest(const FString& Parameters)
{
	const TSharedRef<SCustomDialog> Dialog = SNew(SCustomDialog)
		.Title(FText::FromString("Test dialog."));
	
	const int32 DefaultValue = 5;
	return Dialog->ShowModal(DefaultValue) == DefaultValue;
}

#endif