// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dialog/SMessageDialog.h"

#include "HAL/PlatformApplicationMisc.h"

#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#if WITH_EDITOR
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#else
#include "Widgets/Input/SHyperlink.h"
#endif

void SMessageDialog::Construct(const FArguments& InArgs)
{
	Message = InArgs._Message;
	
	TSharedPtr<SWidget> TextBlockWidget;
	if (InArgs._UseRichText)
	{
		TSharedRef<SRichTextBlock> RichTextBlock = SNew(SRichTextBlock)
			.Text(Message)
			.MinDesiredWidth(InArgs._ContentMinWidth)
			.WrapTextAt(InArgs._WrapMessageAt)
			.Decorators(InArgs._Decorators);

		if (InArgs._DecoratorStyleSet)
		{
			RichTextBlock->SetDecoratorStyleSet(InArgs._DecoratorStyleSet);
		}

		TextBlockWidget = RichTextBlock;
	}
	else
	{
		TextBlockWidget = SNew(STextBlock)
			.Text(Message)
			.MinDesiredWidth(InArgs._ContentMinWidth)
			.WrapTextAt(InArgs._WrapMessageAt);
	}

	const FText CopyToolTipText = NSLOCTEXT("SChoiceDialog", "CopyMessageTooltip", "Copy the text in this message to the clipboard (CTRL+C)");

	SCustomDialog::Construct(SCustomDialog::FArguments()
		.Title(InArgs._Title)
		.Content()
		[
			TextBlockWidget.ToSharedRef()
		]
		.WindowArguments(InArgs._WindowArguments)
		.RootPadding(16.f)
		.Buttons(InArgs._Buttons)
		.AutoCloseOnButtonPress(InArgs._AutoCloseOnButtonPress)
		.Icon(InArgs._Icon)
		.HAlignContent(HAlign_Fill)
		.VAlignContent(VAlign_Fill)
		.IconDesiredSizeOverride(FVector2D(24.f, 24.f))
		.HAlignIcon(HAlign_Left)
		.VAlignIcon(VAlign_Top)
		.ContentAreaPadding(FMargin(16.f, 0.f, 0.f, 0.f))
		.UseScrollBox(InArgs._UseScrollBox)
		.ScrollBoxMaxHeight(InArgs._ScrollBoxMaxHeight)
		.ButtonAreaPadding(FMargin(0.f, 32.f, 0.f, 0.f))
		.OnClosed(InArgs._OnClosed)
		.BeforeButtons()
		[
#if WITH_EDITOR	
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &SMessageDialog::OnCopyMessage)
			.ToolTipText(CopyToolTipText)
			.ContentPadding(2.f)
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Clipboard"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
#else
			SNew(SHyperlink)
			.OnNavigate_Lambda( [this]() { OnCopyMessage(); } ) // using a lambda here to avoid changing the signature of OnCopyMessage
			.Text( NSLOCTEXT("SChoiceDialog", "CopyMessageHyperlink", "Copy") )
			.ToolTipText(CopyToolTipText)
#endif
		]
	);
}

FReply SMessageDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::C && InKeyEvent.IsControlDown())
	{
		CopyMessageToClipboard();
		return FReply::Handled();
	}

	return SCustomDialog::OnKeyDown(MyGeometry, InKeyEvent);
}

FReply SMessageDialog::OnCopyMessage()
{
	CopyMessageToClipboard();
	return FReply::Handled();
}

void SMessageDialog::CopyMessageToClipboard()
{
	FPlatformApplicationMisc::ClipboardCopy(*Message.ToString());
}
