// Copyright Epic Games, Inc. All Rights Reserved.
#include "Filters/SCustomTextFilterDialog.h"

#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "SPrimaryButton.h"

#define LOCTEXT_NAMESPACE "CustomTextFilter"

void SCustomTextFilterDialog::Construct( const FArguments& InArgs )
{
	FilterData = InArgs._FilterData;
	InitialFilterData = FilterData;
	bInEditMode = InArgs._InEditMode;
	bEnableColorEditing = InArgs._InEnableColorEditing;
	OnCreateFilter = InArgs._OnCreateFilter;
	OnDeleteFilter = InArgs._OnDeleteFilter;
	OnCancelClicked = InArgs._OnCancelClicked;
	OnModifyFilter = InArgs._OnModifyFilter;
	OnGetFilterLabels = InArgs._OnGetFilterLabels;
	
	TSharedPtr<SHorizontalBox> ButtonBox = SNew(SHorizontalBox);

	// Buttons for when we are editing a filter
	if(bInEditMode)
	{
	    // Button to modify the current filter
       	ButtonBox->AddSlot()
		.FillWidth(1.0f)
       	.Padding(0.f, 0.f, 16.f, 0.f)
       	.HAlign(HAlign_Right)
       	[
       		SNew(SPrimaryButton)
       		.Text(LOCTEXT("ModifyFilterButton", "Save"))
       		.OnClicked(this, &SCustomTextFilterDialog::OnModifyButtonClicked)
       	];
	
		// Button to delete the current filter
		ButtonBox->AddSlot()
		.AutoWidth()
		.Padding(0.f, 0.f, 16.f, 0.f)
		.HAlign(HAlign_Right)
		[
			SNew(SButton)
			.Text(LOCTEXT("DeleteButton", "Delete"))
			.OnClicked(this, &SCustomTextFilterDialog::OnDeleteButtonClicked)
		];
	}
	else
	{
		// Button to create a new filter and enable it
		ButtonBox->AddSlot()
		.FillWidth(1.0f)
		.Padding(0.f, 0.f, 16.f, 0.f)
		.HAlign(HAlign_Right)
		[
			SNew(SButton)
			.Text(LOCTEXT("CreateAndApplyButton", "Create and Apply"))
			.OnClicked(this, &SCustomTextFilterDialog::OnCreateFilterButtonClicked, true)
		];

		// Button to create a new filter
		ButtonBox->AddSlot()
		.AutoWidth()
		.Padding(0.f, 0.f, 16.f, 0.f)
		.HAlign(HAlign_Right)
		[
			SNew(SPrimaryButton)
			.Text(LOCTEXT("CreateButton", "Create"))
			.OnClicked(this, &SCustomTextFilterDialog::OnCreateFilterButtonClicked, false)
		];
		
	}

	// Button to close the dialog box, common to both modes
	ButtonBox->AddSlot()
	.AutoWidth()
	.Padding(0.f, 0.f, 16.f, 0.f)
	.HAlign(HAlign_Right)
	[
		SNew(SButton)
		.Text(LOCTEXT("CancelButton", "Cancel"))
		.OnClicked(this, &SCustomTextFilterDialog::OnCancelButtonClicked)
	];
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
		[
			SNew(SVerticalBox)

			// Filter Name and Color
			+ SVerticalBox::Slot()
			.Padding(0.f, 38.f, 0.f, 0.f)
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				// Filter Name Text
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(34.f, 0.f, 0.f, 0.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FilterLabelText", "Filter Label"))
					.ToolTipText(LOCTEXT("FilterLabelTooltip", "Display name of the filter"))
				]
				// Filter Name Input Box
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(21.f, 0.f, 0.f, 0.f)
				[
					SNew(SBox)
					.WidthOverride(bEnableColorEditing ? 254.f : 556.f)
					[
						SAssignNew(FilterLabelTextBox, SEditableTextBox)
						.Text_Lambda([this] ()
						{
							return this->FilterData.FilterLabel;
						})
						.OnTextChanged_Lambda([this] (const FText& InText)
						{
							this->FilterData.FilterLabel = InText;
						})
					]
				]

				// Filter Color Text
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(21.f, 0.f, 0.f, 0.f)
				[
					SNew(STextBlock)
					.Visibility(bEnableColorEditing ? EVisibility::Visible : EVisibility::Collapsed)
					.Text(LOCTEXT("FilterColorText", "Color"))
					.ToolTipText(LOCTEXT("FilterColorTooltip", "Display color of the filter button"))
				]
				// Filter Color
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(21.f, 0.f, 0.f, 0.f)
				[
					SAssignNew(ColorBlock, SColorBlock )
					.Visibility(bEnableColorEditing ? EVisibility::Visible : EVisibility::Collapsed)
					.Color_Lambda([this] ()
					{
						return this->FilterData.FilterColor;
					})
					.CornerRadius(FVector4(4.0f, 4.0f, 4.0f, 4.0f))
					.Size(FVector2D(70.0f, 22.0f))
					.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
					.OnMouseButtonDown( this, &SCustomTextFilterDialog::ColorBlock_OnMouseButtonDown )
				]
			]

			// Filter String
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 21.f, 0.f, 0.f)
			[
				SNew(SHorizontalBox)
				// Filter String Text
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(31.f, 0.f, 0.f, 0.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FilterStringText", "Filter String"))
					.ToolTipText(LOCTEXT("FilterStringTooltip", "String literal to query for"))
				]
				// Filter Name Input Box
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(21.f, 0.f, 0.f, 0.f)
				[
					SNew(SBox)
					.WidthOverride(556.f)
					[
						SNew(SEditableTextBox)
						.Text_Lambda([this] ()
						{
							return this->FilterData.FilterString;
						})
						.OnTextChanged_Lambda([this] (const FText& InText)
						{
							this->FilterData.FilterString = InText;
						})
					]
				]
			]

			// Buttons
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Bottom)
			.Padding(0.f, 0.f, 0.f, 16.f)
			[
				ButtonBox.ToSharedRef()
			]
		]
	];
		
}

FReply SCustomTextFilterDialog::ColorBlock_OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		FColorPickerArgs PickerArgs = FColorPickerArgs(FilterData.FilterColor, FOnLinearColorValueChanged::CreateSP(this, &SCustomTextFilterDialog::HandleColorValueChanged));
		PickerArgs.bIsModal = true;
		PickerArgs.ParentWidget = ColorBlock;
		OpenColorPicker(PickerArgs);

		return FReply::Handled();
	}
	else
	{
		return FReply::Unhandled();
	}		
}

void SCustomTextFilterDialog::HandleColorValueChanged(FLinearColor NewValue)
{
	FilterData.FilterColor = NewValue;
}

bool SCustomTextFilterDialog::CheckFilterValidity() const
{
	if(FilterData.FilterLabel.IsEmpty())
	{
		FilterLabelTextBox->SetError(LOCTEXT("EmptyFilterLabelError", "Filter Label cannot be empty"));
		return false;
	}

	if(OnGetFilterLabels.IsBound())
	{
		TArray<FText> ExistingFilterLabels;
		OnGetFilterLabels.Execute(ExistingFilterLabels);

		// Check for duplicate filter labels
		for(const FText& FilterLabel : ExistingFilterLabels)
		{
			/* Special Case: If we are editing a filter and don't change the filter label, it will be considered a duplicate of itself!
			 * To prevent this we check against the original filter label if we are in edit mode
			 */
			if(FilterLabel.EqualTo(FilterData.FilterLabel) && !(bInEditMode && FilterLabel.EqualTo(InitialFilterData.FilterLabel)))
			{
				FilterLabelTextBox->SetError(LOCTEXT("DuplicateFilterLabelError", "A filter with this label already exists!"));
				return false;
			}
		}
	}

	return true;
}

FReply SCustomTextFilterDialog::OnCreateFilterButtonClicked(bool bApplyFilter) const
{
	if(!CheckFilterValidity())
	{
		return FReply::Handled();
	}
	
	OnCreateFilter.ExecuteIfBound(FilterData, bApplyFilter);

	return FReply::Handled();
}

FReply SCustomTextFilterDialog::OnDeleteButtonClicked() const
{
	OnDeleteFilter.ExecuteIfBound();
	
	return FReply::Handled();
}
	
FReply SCustomTextFilterDialog::OnCancelButtonClicked() const
{
	OnCancelClicked.ExecuteIfBound();
	
	return FReply::Handled();
}

FReply SCustomTextFilterDialog::OnModifyButtonClicked() const
{
	if(!CheckFilterValidity())
	{
		return FReply::Handled();
	}
	
	OnModifyFilter.ExecuteIfBound(FilterData);

	return FReply::Handled();
}
#undef LOCTEXT_NAMESPACE
