// Copyright Epic Games, Inc. All Rights Reserved.

#include "Filters/SFilterSearchBox.h"
#include "Framework/Text/TextLayout.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "CustomTextFilter"

/** Internal subclass of SSearchBox, to allow adding the save search and search history buttons to the
 *  Horizontal Box internal to SEditableTextBox
 */
class SFilterSearchBoxImpl : public SSearchBox
{
public:
	
	DECLARE_DELEGATE_RetVal( EVisibility, FGetSearchHistoryVisibility )

	SLATE_BEGIN_ARGS(SFilterSearchBoxImpl)
		: _HintText( LOCTEXT("SearchHint", "Search") )
		, _InitialText()
		, _OnTextChanged()
		, _OnTextCommitted()
		, _DelayChangeNotificationsWhileTyping( true )
		, _DelayChangeNotificationsWhileTypingSeconds( 0.25f )
	{ }

		/** The text displayed in the SearchBox when no text has been entered */
		SLATE_ATTRIBUTE( FText, HintText )

		/** The text displayed in the SearchBox when it's created */
		SLATE_ATTRIBUTE( FText, InitialText )
	
		/** Invoked whenever the text changes */
		SLATE_EVENT( FOnTextChanged, OnTextChanged )

		/** Invoked whenever the text is committed (e.g. user presses enter) */
		SLATE_EVENT( FOnTextCommitted, OnTextCommitted )
	
		/** Whether the SearchBox should delay notifying listeners of text changed events until the user is done typing */
		SLATE_ATTRIBUTE( bool, DelayChangeNotificationsWhileTyping )

		/** If we're delaying change notifications, how many seconds should we wait? */
		SLATE_ATTRIBUTE( float, DelayChangeNotificationsWhileTypingSeconds )

		/** Callback delegate to have first chance handling of the OnKeyDown event */
		SLATE_EVENT(FOnKeyDown, OnKeyDownHandler)

		/** Delegate for when the Plus icon next to an item in the search history is clicked */
		SLATE_EVENT(SFilterSearchBox::FOnSaveSearchClicked, OnSaveSearchClicked)
	
		/** Delegate for when the search history chevron is clicked */
		SLATE_EVENT(FOnClicked, OnSearchHistoryClicked)

		/** Delegate to get the visibility of the search history chevron */
		SLATE_EVENT(FGetSearchHistoryVisibility, GetSearchHistoryVisibility)

		/**
		 * Optional widget to display on the right side of the search box, after the existing buttons.
		 * EXPERIMENTAL: This API is subject to change.
		 */
		PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		SLATE_ARGUMENT(TSharedPtr<SWidget>, ExtraWidget)
		PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	SLATE_END_ARGS()
	
	void Construct( const FArguments& InArgs );
	
	/** Show a + button next to the current search, and set the handler for when that is clicked */
	void SetOnSaveSearchHandler(SFilterSearchBox::FOnSaveSearchClicked InOnSaveSearchHandler);

protected:

	/** Delegate for when the Plus icon next to an item in the search history is clicked */
	SFilterSearchBox::FOnSaveSearchClicked OnSaveSearchClicked;
	
	/** Delegate to get the visibility of the search history chevron */
	FGetSearchHistoryVisibility GetSearchHistoryVisibility;
};

void SFilterSearchBoxImpl::Construct(const FArguments& InArgs)
{
	SSearchBox::Construct( SSearchBox::FArguments()
		.Style(&FAppStyle::GetWidgetStyle<FSearchBoxStyle>("FilterBar.SearchBox"))
		.InitialText(InArgs._InitialText)
		.HintText(InArgs._HintText)
		.OnTextChanged(InArgs._OnTextChanged)
		.OnTextCommitted(InArgs._OnTextCommitted)
		.SelectAllTextWhenFocused( false )
		.DelayChangeNotificationsWhileTyping( InArgs._DelayChangeNotificationsWhileTyping )
		.DelayChangeNotificationsWhileTypingSeconds( InArgs._DelayChangeNotificationsWhileTypingSeconds )
		.OnKeyDownHandler(InArgs._OnKeyDownHandler)
	);

	OnSaveSearchClicked = InArgs._OnSaveSearchClicked;
	GetSearchHistoryVisibility = InArgs._GetSearchHistoryVisibility;

	// + Button to save search
	Box->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.AutoWidth()
		.Padding(-4.f, 0.f, 0.f, 0.f) // Negative padding intentional to have the same padding between all buttons
		[
			// Button to save the currently occurring search 
			SNew(SButton)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("CreateFilterFromCurrentSearchButton")))
			.ContentPadding(0.f)
			.ToolTipText(LOCTEXT("SaveSearchButtonToolTipText", "Save current search as a filter or collection."))
			.Visibility_Lambda([this]()
			{
				// Only visible if there is a search active currently and the OnSaveSearchClicked delegate is bound
				return this->GetText().IsEmpty() || !this->OnSaveSearchClicked.IsBound() ? EVisibility::Collapsed : EVisibility::Visible;
			})
			.ButtonStyle(FAppStyle::Get(), "HoverOnlyButton")
			.OnClicked_Lambda([this]()
			{
				this->OnSaveSearchClicked.ExecuteIfBound(this->GetText());
				return FReply::Handled();
			})
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];

	// Chevron to show search history
	Box->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(0.f, 0.f, 4.f, 0.f)
		[
			// Chevron to open the search history dropdown
			SNew(SButton)
			.ContentPadding(0.f)
			.ButtonStyle(FAppStyle::Get(), "HoverOnlyButton")
			.ClickMethod(EButtonClickMethod::MouseDown)
			.OnClicked(InArgs._OnSearchHistoryClicked)
			.ToolTipText(LOCTEXT("SearchHistoryToolTipText", "Show recent searches."))
			.Visibility_Lambda([this]()
			{
				return this->GetSearchHistoryVisibility.Execute();
			})
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.ChevronDown"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];

	if (InArgs._ExtraWidget.IsValid())
	{
		Box->AddSlot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(0.f, 0.f, 4.f, 0.f)
			[
				InArgs._ExtraWidget.ToSharedRef()
			];
	}
}

void SFilterSearchBoxImpl::SetOnSaveSearchHandler(SFilterSearchBox::FOnSaveSearchClicked InOnSaveSearchHandler)
{
	OnSaveSearchClicked = InOnSaveSearchHandler;
}

void SFilterSearchBox::Construct( const FArguments& InArgs )
{
	MaxSearchHistory = InArgs._MaxSearchHistory;
	OnTextChanged = InArgs._OnTextChanged;
	OnTextCommitted = InArgs._OnTextCommitted;
	bShowSearchHistory = InArgs._ShowSearchHistory;

	if (InArgs._SearchHistory.IsEmpty())
	{
		// Default text shown when there are no items in the search history
		EmptySearchHistoryText = MakeShareable(new FText(LOCTEXT("EmptySearchHistoryText", "The Search History is Empty")));
		SearchHistory.Add(EmptySearchHistoryText);
	}
	else
	{
		// Restore search history if supplied
		SearchHistory.Reserve(InArgs._SearchHistory.Num());
		for (const FText& SearchText : InArgs._SearchHistory)
		{
			SearchHistory.Add(MakeShared<FText>(SearchText));
		}
	}
	
	ChildSlot
	[
		SAssignNew(SearchHistoryBox, SMenuAnchor)
		.Placement(EMenuPlacement::MenuPlacement_ComboBoxRight)
		[
			SAssignNew(SearchBox, SFilterSearchBoxImpl)
			.InitialText(InArgs._InitialText)
			.HintText(InArgs._HintText)
			.OnTextChanged(this, &SFilterSearchBox::HandleTextChanged)
			.OnTextCommitted(this, &SFilterSearchBox::HandleTextCommitted)
			.DelayChangeNotificationsWhileTyping( InArgs._DelayChangeNotificationsWhileTyping )
			.DelayChangeNotificationsWhileTypingSeconds( InArgs._DelayChangeNotificationsWhileTypingSeconds )
			.OnKeyDownHandler(InArgs._OnKeyDownHandler)
			.OnSearchHistoryClicked(this, &SFilterSearchBox::OnClickedSearchHistory)
			.GetSearchHistoryVisibility(this, &SFilterSearchBox::GetSearchHistoryVisibility)
			.OnSaveSearchClicked(InArgs._OnSaveSearchClicked)
			.ExtraWidget(InArgs._ExtraWidget)
		]
		.MenuContent
		(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Menu.Background"))
			.Padding( FMargin(2.f) )
			[
				SAssignNew(SearchHistoryListView, SListView< TSharedPtr<FText> >)
				.ListItemsSource(&SearchHistory)
				.SelectionMode( ESelectionMode::Single )
				.OnGenerateRow(this, &SFilterSearchBox::MakeSearchHistoryRowWidget)
				.OnSelectionChanged( this, &SFilterSearchBox::OnSelectionChanged)
				.ScrollbarDragFocusCause(EFocusCause::SetDirectly)
			]
		)
	];
}

bool SFilterSearchBox::SupportsKeyboardFocus() const
{
	return SearchBox->SupportsKeyboardFocus();
}

bool SFilterSearchBox::HasKeyboardFocus() const
{
	// Since keyboard focus is forwarded to our editable text, we will test it instead
	return SearchBox->HasKeyboardFocus();
}

FReply SFilterSearchBox::OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent )
{
	// Forward keyboard focus to our editable text widget
	return FReply::Handled().SetUserFocus(SearchBox.ToSharedRef(), InFocusEvent.GetCause());
}

/** Handler for when text in the editable text box changed */
void SFilterSearchBox::HandleTextChanged(const FText& NewText)
{
	OnTextChanged.ExecuteIfBound(NewText);
}

/** Handler for when text in the editable text box changed */
void SFilterSearchBox::HandleTextCommitted(const FText& NewText, ETextCommit::Type CommitType)
{
	// Set the text and execute the delegate
	SetText(NewText);
	OnTextCommitted.ExecuteIfBound(NewText, CommitType);
	
	UpdateSearchHistory(NewText);
}

void SFilterSearchBox::SetText(const TAttribute< FText >& InNewText)
{
	SearchBox->SetText(InNewText);
}

FText SFilterSearchBox::GetText() const
{
	return SearchBox->GetText();
}

TArray<FText> SFilterSearchBox::GetSearchHistory() const
{
	TArray<FText> Results;

	// If there is only one item, check that it's not the placeholder empty search text
	if (SearchHistory.Num() == 1 && SearchHistory[0] == EmptySearchHistoryText)
	{
		return Results;
	}

	Results.Reserve(SearchHistory.Num());
	for (const TSharedPtr<FText>& SearchHistoryText : SearchHistory)
	{
		Results.Add(*SearchHistoryText.Get());
	}
	return Results;
}

void SFilterSearchBox::SetError( const FText& InError )
{
	SearchBox->SetError(InError);
}

void SFilterSearchBox::SetError( const FString& InError )
{
	SearchBox->SetError(InError);
}

void SFilterSearchBox::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	// Update the search history on focus loss (for cases with SearchBoxes that don't commit on Enter)
	UpdateSearchHistory(SearchBox->GetText());
}

/** Called by SListView when the selection changes in the search history list */
void SFilterSearchBox::OnSelectionChanged( TSharedPtr<FText> NewValue, ESelectInfo::Type SelectInfo )
{
	/* Make sure the user can only select an item in the history using the mouse, and that they cannot select the
	 * Placeholder text for empty search history
	 */
	if(SelectInfo != ESelectInfo::OnNavigation && NewValue && NewValue != EmptySearchHistoryText)
	{
		SearchBox->SetText(*NewValue);
		SearchHistoryBox->SetIsOpen(false);
	}
}

TSharedRef<ITableRow> SFilterSearchBox::MakeSearchHistoryRowWidget(TSharedPtr<FText> SearchText, const TSharedRef<STableViewBase>& OwnerTable)
{
	bool bIsEmptySearchHistory = (SearchText == EmptySearchHistoryText);
	EHorizontalAlignment TextAlignment = (bIsEmptySearchHistory) ? HAlign_Center : HAlign_Left;

	TSharedPtr<SHorizontalBox> RowWidget = SNew(SHorizontalBox);

	// The actual search text
	RowWidget->AddSlot()
	.HAlign(TextAlignment)
	.VAlign(VAlign_Center)
	.MaxWidth(500.0f)
	.FillWidth(1.0f)
	[
		SNew(STextBlock)
		.Text(*SearchText.Get())
		.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
	];
	
	return SNew(STableRow< TSharedPtr<FString> >, OwnerTable)
			.ShowSelection(!bIsEmptySearchHistory)
			[
				RowWidget.ToSharedRef()
			];
}

void SFilterSearchBox::UpdateSearchHistory(const FText &NewText)
{
	// Don't save empty searches
	if(NewText.IsEmpty())
	{
		return;
	}

	// If there is only one item, and it is the placeholder empty search text, remove it
	if(SearchHistory.Num() == 1 && SearchHistory[0] == EmptySearchHistoryText)
	{
		SearchHistory.Empty();
	}

	// Remove any existing occurances of the current search text, we will re-add it to the top if so
	SearchHistory.RemoveAll([&NewText](TSharedPtr<FText> SearchHistoryText)
	{
		if(SearchHistoryText->CompareTo(NewText) == 0)
		{
			return true;
		}
		return false;
	});

	// Insert the current search as the most recent in the history
	SearchHistory.Insert(MakeShareable(new FText(NewText)), 0);

	// Prune old entries until we are at the Max Search History limit
	while( SearchHistory.Num() > MaxSearchHistory)
	{
		SearchHistory.RemoveAt( SearchHistory.Num()-1 );
	}

	SearchHistoryListView->RequestListRefresh();
}

FReply SFilterSearchBox::OnClickedSearchHistory()
{
	if(SearchHistoryBox->ShouldOpenDueToClick() && !SearchHistory.IsEmpty())
	{
		SearchHistoryBox->SetIsOpen(true);
		SearchHistoryListView->ClearSelection();
	}
	else
	{
		SearchHistoryBox->SetIsOpen(false);
	}
	
	return FReply::Handled();
}

void SFilterSearchBox::SetOnSaveSearchHandler(FOnSaveSearchClicked InOnSaveSearchHandler)
{
	SearchBox->SetOnSaveSearchHandler(InOnSaveSearchHandler);
}

EVisibility SFilterSearchBox::GetSearchHistoryVisibility() const
{
	return bShowSearchHistory.Get() ? EVisibility::Visible : EVisibility::Collapsed;
}

FTextSelection SFilterSearchBox::GetSelection() const
{
	return SearchBox->GetSelection();
}

void SFilterSearchBox::SelectText(const FTextLocation& InSelectionStart, const FTextLocation& InCursorLocation)
{
	SearchBox->SelectText(InSelectionStart, InCursorLocation);
}

#undef LOCTEXT_NAMESPACE
