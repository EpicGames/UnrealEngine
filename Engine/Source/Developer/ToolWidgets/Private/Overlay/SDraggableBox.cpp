// Copyright Epic Games, Inc. All Rights Reserved.

#include "Overlay/SDraggableBox.h"

#include "Framework/Application/SlateApplication.h"
#include "Overlay/SDraggableBoxOverlay.h"
#include "Input/DragAndDrop.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"

namespace UE::ToolWidgets
{
namespace DraggableBoxDetail
{
/** A drag/drop operation used by SDraggableBox. */
class FDraggableBoxUIDragOperation : public FDragDropOperation
{
public:
	FDraggableBoxUIDragOperation(const TSharedRef<SDraggableBox> InDraggableBox,
		const SDraggableBox::FDragInfo& InDragInfo)
		: DraggableBoxWeak(InDraggableBox)
		, DragInfo(InDragInfo)
	{
	}

	virtual ~FDraggableBoxUIDragOperation() override = default;

	//~ Begin FDragDropOperation
	virtual void OnDragged(const FDragDropEvent& InDragDropEvent)
	{
		if (TSharedPtr<SDraggableBox> DraggableBox = DraggableBoxWeak.Pin())
		{
			DraggableBox->OnDragUpdate(InDragDropEvent, DragInfo, /* Dropped */ false);
		}
	}

	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& InMouseEvent)
	{
		if (TSharedPtr<SDraggableBox> DraggableBox = DraggableBoxWeak.Pin())
		{
			DraggableBox->OnDragUpdate(InMouseEvent, DragInfo, /* Dropped */ true);
		}
	}
	//~ End FDragDropOperation

protected:
	TWeakPtr<SDraggableBox> DraggableBoxWeak;
	SDraggableBox::FDragInfo DragInfo;
};
}

void SDraggableBox::Construct(const FArguments& InArgs, const TSharedRef<SDraggableBoxOverlay>& InDraggableOverlay)
{
	DraggableOverlayWeak = InDraggableOverlay;
	InnerWidget = InArgs._Content.Widget;
	DraggableAttr = InArgs._Draggable;
	DragOverridesCenterAlignmentAttr = InArgs._DragOverridesCenterAlignment;
	OnUserDraggedToNewPositionDelegate = InArgs._OnUserDraggedToNewPosition;

	ResizeEdges           = InArgs._Resizable;
	ResizeHandleThickness = InArgs._ResizeHandleThickness;
	OnUserResizedDelegate = InArgs._OnUserResized;

	ChildSlot
	[
		// Wrap content in an SBox so we can impose explicit WidthOverride/HeightOverride during resize.
		// When no explicit size is set, the overrides return FOptionalSize() (unset) and the SBox is a pass-through.
		SAssignNew(ContentSizeBox, SBox)
		.WidthOverride(this, &SDraggableBox::GetWidthOverride)
		.HeightOverride(this, &SDraggableBox::GetHeightOverride)
		[
			InArgs._Content.Widget
		]
	];
}

void SDraggableBox::OnDragUpdate(const FPointerEvent& InMouseEvent, const FDragInfo& InDragInfo, bool bInDropped)
{
	const TSharedPtr<SDraggableBoxOverlay> DraggableOverlay = DraggableOverlayWeak.Pin();
	if (!DraggableOverlay.IsValid() || !DraggableAttr.Get())
	{
		return;
	}

	const FGeometry& MyGeometry = DraggableOverlay->GetTickSpaceGeometry();
	const FVector2f MouseOffset = (InMouseEvent.GetScreenSpacePosition() - InDragInfo.OriginalMousePosition)
		* (MyGeometry.GetLocalSize() / MyGeometry.GetAbsoluteSize());
	
	FVector2f NewAlignmentOffset = InDragInfo.OriginalAlignmentOffset;
	EHorizontalAlignment NewHorizontalAlignment = InDragInfo.OriginalHorizontalAlignment;
	EVerticalAlignment NewVerticalAlignment = InDragInfo.OriginalVerticalAlignment;
	
	// If we allow drag to override center alignment, compute the current centered alignment offset.
	FVector2f CenterOffset(0.f, 0.f);
	if (DragOverridesCenterAlignmentAttr.Get())
	{
		CenterOffset = GetCenterBoxAlignmentOffset();
	}

	switch (InDragInfo.OriginalHorizontalAlignment)
	{
	case EHorizontalAlignment::HAlign_Left:
		NewAlignmentOffset.X += MouseOffset.X;
		break;

	case EHorizontalAlignment::HAlign_Right:
		NewAlignmentOffset.X -= MouseOffset.X;
		break;
		
	case EHorizontalAlignment::HAlign_Center:
		if (DragOverridesCenterAlignmentAttr.Get())
		{
			NewAlignmentOffset.X = CenterOffset.X + MouseOffset.X;
			NewHorizontalAlignment = HAlign_Left;
		}
		break;

	default:
		// Do nothing
		break;
	}

	switch (InDragInfo.OriginalVerticalAlignment)
	{
	case EVerticalAlignment::VAlign_Top:
		NewAlignmentOffset.Y += MouseOffset.Y;
		break;

	case EVerticalAlignment::VAlign_Bottom:
		NewAlignmentOffset.Y -= MouseOffset.Y;
		break;
		
	case EVerticalAlignment::VAlign_Center:
		if (DragOverridesCenterAlignmentAttr.Get())
		{
			NewAlignmentOffset.Y = CenterOffset.Y + MouseOffset.Y;
			NewVerticalAlignment = VAlign_Top;
		}
		break;

	default:
		// Do nothing
		break;
	}

	DraggableOverlay->SetBoxHorizontalAlignment(NewHorizontalAlignment);
	DraggableOverlay->SetBoxVerticalAlignment(NewVerticalAlignment);
	DraggableOverlay->SetBoxAlignmentOffset(NewAlignmentOffset);

	if (bInDropped)
	{
		OnUserDraggedToNewPositionDelegate.ExecuteIfBound();
	}
}

FOptionalSize SDraggableBox::GetWidthOverride() const
{
	if (const TSharedPtr<SDraggableBoxOverlay> Overlay = DraggableOverlayWeak.Pin())
	{
		TOptional<float> Width = Overlay->GetWidthOverride();
		if (Width.IsSet())
		{
			return FOptionalSize(Width.GetValue());
		}
	}
	return FOptionalSize();
}

FOptionalSize SDraggableBox::GetHeightOverride() const
{
	if (const TSharedPtr<SDraggableBoxOverlay> Overlay = DraggableOverlayWeak.Pin())
	{
		TOptional<float> Height = Overlay->GetHeightOverride();
		if (Height.IsSet())
		{
			return FOptionalSize(Height.GetValue());
		}
	}
	return FOptionalSize();
}

EResizeEdges SDraggableBox::GetEdgeAtLocalPosition(FVector2f LocalPos, FVector2f WidgetSize) const
{
	if (ResizeEdges == EResizeEdges::None)
	{
		return EResizeEdges::None;
	}

	EResizeEdges HitEdge = EResizeEdges::None;
	if (EnumHasAnyFlags(ResizeEdges, EResizeEdges::Left)   && LocalPos.X <= ResizeHandleThickness)
	{
		HitEdge |= EResizeEdges::Left;
	}
	if (EnumHasAnyFlags(ResizeEdges, EResizeEdges::Right)  && LocalPos.X >= WidgetSize.X - ResizeHandleThickness)
	{
		HitEdge |= EResizeEdges::Right;
	}
	if (EnumHasAnyFlags(ResizeEdges, EResizeEdges::Top)    && LocalPos.Y <= ResizeHandleThickness)
	{
		HitEdge |= EResizeEdges::Top;
	}
	if (EnumHasAnyFlags(ResizeEdges, EResizeEdges::Bottom) && LocalPos.Y >= WidgetSize.Y - ResizeHandleThickness)
	{
		HitEdge |= EResizeEdges::Bottom;
	}
	return HitEdge;
}

FCursorReply SDraggableBox::GetCursorForEdge(EResizeEdges Edge) const
{
	const bool bH = EnumHasAnyFlags(Edge, EResizeEdges::Left  | EResizeEdges::Right);
	const bool bV = EnumHasAnyFlags(Edge, EResizeEdges::Top   | EResizeEdges::Bottom);

	if (bH && bV)
	{
		// Top-Left or Bottom-Right = SE diagonal; Top-Right or Bottom-Left = SW
		const bool bSE = (EnumHasAnyFlags(Edge, EResizeEdges::Top)    && EnumHasAnyFlags(Edge, EResizeEdges::Left))
		              || (EnumHasAnyFlags(Edge, EResizeEdges::Bottom)  && EnumHasAnyFlags(Edge, EResizeEdges::Right));
		return FCursorReply::Cursor(bSE ? EMouseCursor::ResizeSouthEast : EMouseCursor::ResizeSouthWest);
	}
	if (bH)
	{
		return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
	}
	if (bV)
	{
		return FCursorReply::Cursor(EMouseCursor::ResizeUpDown);
	}
	return FCursorReply::Unhandled();
}

FReply SDraggableBox::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && ResizeEdges != EResizeEdges::None)
	{
		const FVector2f LocalPos = InGeometry.AbsoluteToLocal(InMouseEvent.GetScreenSpacePosition());
		const EResizeEdges Edge = GetEdgeAtLocalPosition(LocalPos, InGeometry.GetLocalSize());
		if (Edge != EResizeEdges::None)
		{
			ActiveResizeEdge    = Edge;
			ResizeStartMousePos = InMouseEvent.GetScreenSpacePosition();

			if (const TSharedPtr<SDraggableBoxOverlay> Overlay = DraggableOverlayWeak.Pin())
			{
				// Seed each axis independently: use the explicit override if set, otherwise the current rendered size.
				const FVector2f CurrentGeomSize = FVector2f(GetTickSpaceGeometry().GetLocalSize());
				ResizeStartBoxSize.X       = Overlay->GetWidthOverride().Get(CurrentGeomSize.X);
				ResizeStartBoxSize.Y       = Overlay->GetHeightOverride().Get(CurrentGeomSize.Y);
				ResizeStartAlignmentOffset = Overlay->GetBoxAlignmentOffset();
			}

			return FReply::Handled().CaptureMouse(SharedThis(this));
		}
	}

	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		// Not on a resize edge - fall through to the existing drag-to-reposition path.
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	}

	return FReply::Unhandled();
}

FReply SDraggableBox::OnMouseButtonUp(const FGeometry& /*MyGeometry*/, const FPointerEvent& MouseEvent)
{
	if (HasMouseCapture() && ActiveResizeEdge != EResizeEdges::None
		&& MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		ActiveResizeEdge = EResizeEdges::None;
		OnUserResizedDelegate.ExecuteIfBound();
		return FReply::Handled().ReleaseMouseCapture();
	}
	return FReply::Unhandled();
}

FReply SDraggableBox::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const FVector2f LocalPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	if (HasMouseCapture() && ActiveResizeEdge != EResizeEdges::None)
	{
		if (const TSharedPtr<SDraggableBoxOverlay> Overlay = DraggableOverlayWeak.Pin())
		{
			// Convert the screen-space delta to local space (accounting for DPI scale).
			const FGeometry& OverlayGeometry = Overlay->GetTickSpaceGeometry();
			const FVector2f LocalScale = OverlayGeometry.GetLocalSize() / OverlayGeometry.GetAbsoluteSize();
			if (!LocalScale.ContainsNaN()) // Don't call this during construction because the geometry is not initialized, yet. 
			{
				const FVector2f MouseDelta = (MouseEvent.GetScreenSpacePosition() - ResizeStartMousePos) * LocalScale;

				FVector2f NewSize   = ResizeStartBoxSize;
				FVector2f NewOffset = ResizeStartAlignmentOffset;

				// Horizontal resize
				if (EnumHasAnyFlags(ActiveResizeEdge, EResizeEdges::Right))
				{
					// Use Set/GetWidthOverride to clamp the width.
					Overlay->SetWidthOverride(ResizeStartBoxSize.X + MouseDelta.X);
					NewSize.X = Overlay->GetWidthOverride().GetValue();
					
					// HAlign_Right: the right edge is the anchor; as the box widens the right
					// edge moves toward the overlay boundary, so Padding.Right decreases.
					if (Overlay->GetBoxHorizontalAlignment() == HAlign_Right)
					{
						const float ActualDelta = NewSize.X - ResizeStartBoxSize.X;
						NewOffset.X = ResizeStartAlignmentOffset.X - ActualDelta;
					}
					// HAlign_Left: left edge is the anchor - offset unchanged.
				}
				if (EnumHasAnyFlags(ActiveResizeEdge, EResizeEdges::Left))
				{
					// Moving left (negative delta) grows the box.
					// Use Set/GetWidthOverride to clamp the width.
					Overlay->SetWidthOverride(ResizeStartBoxSize.X - MouseDelta.X);
					NewSize.X = Overlay->GetWidthOverride().GetValue();
					
					// HAlign_Left: the left edge follows the mouse; as the box widens the left
					// edge moves away from the overlay boundary, so Padding.Left decreases.
					if (Overlay->GetBoxHorizontalAlignment() == HAlign_Left)
					{
						const float ActualDelta = NewSize.X - ResizeStartBoxSize.X;
						NewOffset.X = ResizeStartAlignmentOffset.X - ActualDelta;
					}
					// HAlign_Right: right edge is the anchor - offset unchanged.
				}

				// Vertical resize
				if (EnumHasAnyFlags(ActiveResizeEdge, EResizeEdges::Bottom))
				{
					// Use Set/GetHeightOverride to clamp the height
					Overlay->SetHeightOverride(ResizeStartBoxSize.Y + MouseDelta.Y);
					NewSize.Y = Overlay->GetHeightOverride().GetValue();
					
					// VAlign_Bottom: the bottom edge follows the mouse; as the box grows the
					// bottom edge moves toward the overlay boundary, so Padding.Bottom decreases.
					if (Overlay->GetBoxVerticalAlignment() == VAlign_Bottom)
					{
						const float ActualDelta = NewSize.Y - ResizeStartBoxSize.Y;
						NewOffset.Y = ResizeStartAlignmentOffset.Y - ActualDelta;
					}
					// VAlign_Top: top edge is the anchor - offset unchanged.
				}
				if (EnumHasAnyFlags(ActiveResizeEdge, EResizeEdges::Top))
				{
					// Moving up (negative delta) grows the box.
					// Use Set/GetHeightOverride to clamp the height
					Overlay->SetHeightOverride(ResizeStartBoxSize.Y - MouseDelta.Y);
					NewSize.Y = Overlay->GetHeightOverride().GetValue();
					
					// VAlign_Top: the top edge follows the mouse; as the box grows the top
					// edge moves away from the overlay boundary, so Padding.Top decreases.
					if (Overlay->GetBoxVerticalAlignment() == VAlign_Top)
					{
						const float ActualDelta = NewSize.Y - ResizeStartBoxSize.Y;
						NewOffset.Y = ResizeStartAlignmentOffset.Y - ActualDelta;
					}
					// VAlign_Bottom: bottom edge is the anchor - offset unchanged.
				}

				// Suppress RecomputeAnchorPoints during resize: flipping HAlign/VAlign mid-drag
				// would cause the dragged edge to jump and grow in the wrong direction.
				Overlay->SetBoxAlignmentOffset(NewOffset, /* bInRecomputeAnchorPoints = */ false);
			}
		}
		return FReply::Handled();
	}

	// Not resizing - update hover edge so OnCursorQuery can reflect it.
	HoveredResizeEdge = GetEdgeAtLocalPosition(LocalPos, MyGeometry.GetLocalSize());
	return FReply::Unhandled();
}

void SDraggableBox::OnMouseLeave(const FPointerEvent& /*MouseEvent*/)
{
	// Clear hover highlight when the cursor leaves, but keep ActiveResizeEdge if a drag is in progress.
	if (ActiveResizeEdge == EResizeEdges::None)
	{
		HoveredResizeEdge = EResizeEdges::None;
	}
}

FCursorReply SDraggableBox::OnCursorQuery(const FGeometry& /*MyGeometry*/, const FPointerEvent& /*CursorEvent*/) const
{
	// While actively resizing keep the cursor locked to the resize type; otherwise reflect hover state.
	const EResizeEdges EdgeForCursor = (ActiveResizeEdge != EResizeEdges::None) ? ActiveResizeEdge : HoveredResizeEdge;
	return GetCursorForEdge(EdgeForCursor);
}

FReply SDraggableBox::OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	const TSharedPtr<SDraggableBoxOverlay> DraggableOverlay = DraggableOverlayWeak.Pin();
	if (DraggableOverlay && DraggableAttr.Get())
	{
		const FDragInfo DragInfo
		{
			DraggableOverlay->GetBoxHorizontalAlignment(),
			DraggableOverlay->GetBoxVerticalAlignment(),
			DraggableOverlay->GetBoxAlignmentOffset(),
			InMouseEvent.GetScreenSpacePosition()
		};
		using namespace DraggableBoxDetail;
		const TSharedRef<FDraggableBoxUIDragOperation> DragDropOperation = MakeShared<FDraggableBoxUIDragOperation>(
			SharedThis(this), DragInfo
		);
		return FReply::Handled().BeginDragDrop(DragDropOperation);
	}
	return FReply::Unhandled();
}
	
FVector2f SDraggableBox::GetCenterBoxAlignmentOffset() const
{
	const TSharedPtr<SDraggableBoxOverlay> DraggableOverlay = DraggableOverlayWeak.Pin();
	if (!DraggableOverlay.IsValid() || !DraggableAttr.Get())
	{
		return FVector2f::ZeroVector;
	}
	
	const FGeometry& MyGeometry = DraggableOverlay->GetTickSpaceGeometry();
	const FVector2f MyGeometryScale = MyGeometry.GetLocalSize() / MyGeometry.GetAbsoluteSize();

	const FVector2f BoxSize = GetTickSpaceGeometry().GetAbsoluteSize();
	const FVector2f AvailableSpace = (MyGeometry.GetAbsoluteSize() - BoxSize);
	const FVector2f MidPoint = AvailableSpace * 0.5f * MyGeometryScale;
	if (!ensure(!MidPoint.ContainsNaN())) // Don't call this during construction because the geometry is not initialized, yet. 
	{
		return FVector2f::ZeroVector;
	}
	
	FVector2f CenterOffset;
	CenterOffset.X = FMath::Max(MidPoint.X, 0.f);
	CenterOffset.Y = FMath::Max(MidPoint.Y, 0.f);
	return CenterOffset;
}
}