// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Overlay/SDraggableBoxOverlay.h"     // for EResizeEdges
#include "Types/SlateStructs.h"               // for FOptionalSize

class SBox;

namespace UE::ToolWidgets
{
	
/**
 * A widget for the draggable box itself, which requires its parent to handle its positioning in
 * response to the drag.
 *
 * Users probably shouldn't use this class directly; rather, they should use SDraggableBoxOverlay,
 * which will put its contents into a draggable box and properly handle the dragging without the
 * user having to set it up.
 */
class SDraggableBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDraggableBox)
		: _Draggable(true)
		, _DragOverridesCenterAlignment(false)
		, _Resizable(EResizeEdges::None)
		, _ResizeHandleThickness(5.f)
	{}
	SLATE_DEFAULT_SLOT(FArguments, Content)

	/** Whether this widget should be allowed to be dragged. */
	SLATE_ATTRIBUTE(bool, Draggable)

	/** Whether this widget can override center alignment when dragged. */
	SLATE_ATTRIBUTE(bool, DragOverridesCenterAlignment)
	
	/** Invoked when the user has finished dragging the widget to a new position. */
	SLATE_EVENT(FSimpleDelegate, OnUserDraggedToNewPosition)

	/** Which edges the user is allowed to resize by dragging. Forwarded from SDraggableBoxOverlay. */
	SLATE_ARGUMENT(EResizeEdges, Resizable)
		
	/** Thickness (in local pixels) of the invisible resize hit-target at each enabled edge. */
	SLATE_ARGUMENT(float, ResizeHandleThickness)
		
	/** Invoked when the user finishes a resize drag. */
	SLATE_EVENT(FSimpleDelegate, OnUserResized)
		
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SDraggableBoxOverlay>& InDraggableOverlay);

	struct FDragInfo
	{
		EHorizontalAlignment OriginalHorizontalAlignment = HAlign_Left;
		EVerticalAlignment OriginalVerticalAlignment = VAlign_Bottom;
		FVector2f OriginalAlignmentOffset = FVector2f::ZeroVector;
		FVector2f OriginalMousePosition;
	};

	void OnDragUpdate(const FPointerEvent& InMouseEvent, const FDragInfo& InDragInfo, bool bInDropped);

	//~ Begin SWidget
	FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	void   OnMouseLeave(const FPointerEvent& MouseEvent) override;
	FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	//~ End SWidget

protected:

	TWeakPtr<SDraggableBoxOverlay> DraggableOverlayWeak;
	TSharedPtr<SWidget> InnerWidget;

private:

	/** Whether this widget should be allowed to be dragged. */
	TAttribute<bool> DraggableAttr;

	/** Whether this widget can override center alignment when dragged. */
	TAttribute<bool> DragOverridesCenterAlignmentAttr;

	/** The content you want to be able to drag around in the parent widget. */
	FSimpleDelegate OnUserDraggedToNewPositionDelegate;

	FVector2f GetCenterBoxAlignmentOffset() const;

	// Resize support

	/** SBox wrapper around the content that provides explicit WidthOverride/HeightOverride when resizing. */
	TSharedPtr<SBox> ContentSizeBox;

	EResizeEdges    ResizeEdges            = EResizeEdges::None;
	float           ResizeHandleThickness  = 5.f;
	FSimpleDelegate OnUserResizedDelegate;

	/** Edge(s) the cursor is currently hovering over (no button held). Used for cursor feedback. */
	EResizeEdges HoveredResizeEdge = EResizeEdges::None;
	/** Edge(s) being actively resized (mouse captured). */
	EResizeEdges ActiveResizeEdge  = EResizeEdges::None;

	/** Screen-space mouse position when the resize began. */
	FVector2f ResizeStartMousePos;
	/** Box size (local) at the start of the resize. */
	FVector2f ResizeStartBoxSize;
	/** Overlay alignment offset at the start of the resize (for Left/Top edge adjustment). */
	FVector2f ResizeStartAlignmentOffset;

	/** Returns which configured edge(s) the given local position falls within. */
	EResizeEdges GetEdgeAtLocalPosition(FVector2f LocalPos, FVector2f WidgetSize) const;
	/** Returns the appropriate resize cursor reply for the given edge flags. */
	FCursorReply GetCursorForEdge(EResizeEdges Edge) const;

	/** Bound to ContentSizeBox.WidthOverride - reads ExplicitBoxSize from the overlay. */
	FOptionalSize GetWidthOverride() const;
	/** Bound to ContentSizeBox.HeightOverride - reads ExplicitBoxSize from the overlay. */
	FOptionalSize GetHeightOverride() const;
};
}