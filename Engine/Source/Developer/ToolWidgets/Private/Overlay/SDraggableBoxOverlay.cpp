// Copyright Epic Games, Inc. All Rights Reserved.

#include "Overlay/SDraggableBoxOverlay.h"

#include "Framework/Application/SlateApplication.h"
#include "Overlay/DragBoxPosition.h"
#include "Overlay/SDraggableBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SWindow.h"

namespace UE::ToolWidgets
{
namespace DraggableBoxDetail
{
	static constexpr float DraggableBorderArea = 3.f;
}

void SDraggableBoxOverlay::Construct(const FArguments& InArgs)
{
	HorizontalAlignment = InArgs._HAlign;
	VerticalAlignment = InArgs._VAlign;
	MinimumBoxWidth = InArgs._MinimumBoxWidth;
	MinimumBoxHeight = InArgs._MinimumBoxHeight;
	MaximumBoxWidth = InArgs._MaximumBoxWidth;
	MaximumBoxHeight = InArgs._MaximumBoxHeight;
	
	ChildSlot
	[
		SAssignNew(Container, SBox)
		.HAlign(HorizontalAlignment)
		.VAlign(VerticalAlignment)
		[
			SAssignNew(DraggableBox, SDraggableBox, SharedThis(this))
			.Draggable(InArgs._Draggable)
			.DragOverridesCenterAlignment(InArgs._DragOverridesCenterAlignment)
			.OnUserDraggedToNewPosition(InArgs._OnUserDraggedToNewPosition)
			.Resizable(InArgs._Resizable)
			.ResizeHandleThickness(InArgs._ResizeHandleThickness)
			.OnUserResized(InArgs._OnUserResized)
			.Clipping(EWidgetClipping::ClipToBounds)
			[
				InArgs._Content.Widget
			]
		]
	];

	SetBoxAlignmentOffset(InArgs._InitialAlignmentOffset, false);
	SetWidthOverride(InArgs._WidthOverride);
	SetHeightOverride(InArgs._HeightOverride);
}

FVector2f SDraggableBoxOverlay::GetBoxAlignmentOffset() const
{
	FVector2f AlignmentOffset = FVector2f::ZeroVector;

	if (Container.IsValid())
	{
		switch (HorizontalAlignment)
		{
		case EHorizontalAlignment::HAlign_Left:
			AlignmentOffset.X = Padding.Left;
			break;
		case EHorizontalAlignment::HAlign_Right:
			AlignmentOffset.X = Padding.Right;
			break;
		default: break;
		}

		switch (VerticalAlignment)
		{
		case EVerticalAlignment::VAlign_Top:
			AlignmentOffset.Y = Padding.Top;
			break;
		case EVerticalAlignment::VAlign_Bottom:
			AlignmentOffset.Y = Padding.Bottom;
			break;
		default: break;
		}
	}

	return AlignmentOffset;
}

void SDraggableBoxOverlay::SetBoxAlignmentOffset(const FVector2f& InOffset, bool bInRecomputeAnchorPoints)
{
	if (!Container.IsValid() || !DraggableBox.IsValid())
	{
		return;
	}

	FVector2f ConstrainedOffset = {
		FMath::Max(InOffset.X, DraggableBoxDetail::DraggableBorderArea),
		FMath::Max(InOffset.Y, DraggableBoxDetail::DraggableBorderArea)
	};
	if (bInRecomputeAnchorPoints)
	{
		ConstrainedOffset = RecomputeAnchorPoints(ConstrainedOffset);
	}

	switch (HorizontalAlignment)
	{
	case EHorizontalAlignment::HAlign_Left:
		Padding.Left = ConstrainedOffset.X;
		Padding.Right = 0.f;
		break;

	case EHorizontalAlignment::HAlign_Right:
		Padding.Left = 0.f;
		Padding.Right = ConstrainedOffset.X;
		break;
		
	case EHorizontalAlignment::HAlign_Center:
	case EHorizontalAlignment::HAlign_Fill:
		// Clear out any padding from prior alignments (Left/Right).
		Padding.Left = 0.f;
		Padding.Right = 0.f;
		break;

	default: break;
	}

	switch (VerticalAlignment)
	{
	case EVerticalAlignment::VAlign_Top:
		Padding.Top = ConstrainedOffset.Y;
		Padding.Bottom = 0.f;
		break;

	case EVerticalAlignment::VAlign_Bottom:
		Padding.Top = 0.f;
		Padding.Bottom = ConstrainedOffset.Y;
		break;
		
	case EVerticalAlignment::VAlign_Center:
	case EVerticalAlignment::VAlign_Fill:
		// Clear out any padding from prior alignments (Left/Right).
		Padding.Top = 0.f;
		Padding.Bottom = 0.f;
		break;

	default: break;
	}

	if (Container.IsValid())
	{
		Container->SetPadding(Padding);
	}
}

EHorizontalAlignment SDraggableBoxOverlay::GetBoxHorizontalAlignment() const
{
	return HorizontalAlignment;
}

void SDraggableBoxOverlay::SetBoxHorizontalAlignment(EHorizontalAlignment InAlignment)
{
	HorizontalAlignment = InAlignment;

	if (Container.IsValid())
	{
		Container->SetHAlign(InAlignment);
	}
}

EVerticalAlignment SDraggableBoxOverlay::GetBoxVerticalAlignment() const
{
	return VerticalAlignment;
}

void SDraggableBoxOverlay::SetBoxVerticalAlignment(EVerticalAlignment InAlignment)
{
	VerticalAlignment = InAlignment;

	if (Container.IsValid())
	{
		Container->SetVAlign(InAlignment);
	}
}

FMargin SDraggableBoxOverlay::GetPadding() const
{
	return Padding;
}

FVector2f SDraggableBoxOverlay::RecomputeAnchorPoints(const FVector2f& InOffset)
{
	const FGeometry& MyGeometry = GetTickSpaceGeometry();

	const FVector2f AvailableSpace = (MyGeometry.GetAbsoluteSize() - DraggableBox->GetTickSpaceGeometry().GetAbsoluteSize())
		* (MyGeometry.GetLocalSize() / MyGeometry.GetAbsoluteSize());
	const FVector2f MidPoint = AvailableSpace * 0.5f;
	if (!ensure(!MidPoint.ContainsNaN())) // Don't call this during construction because the geometry is not initialized, yet. 
	{
		return InOffset;
	}

	FVector2f ConstrainedOffset = InOffset;
	ConstrainedOffset.X = FMath::Min(ConstrainedOffset.X, AvailableSpace.X);
	ConstrainedOffset.Y = FMath::Min(ConstrainedOffset.Y, AvailableSpace.Y);
	if (ConstrainedOffset.X > MidPoint.X)
	{
		ConstrainedOffset.X = FMath::Max(AvailableSpace.X - ConstrainedOffset.X, DraggableBoxDetail::DraggableBorderArea); // Circle value around

		switch (HorizontalAlignment)
		{
		case EHorizontalAlignment::HAlign_Left:
			SetBoxHorizontalAlignment(EHorizontalAlignment::HAlign_Right);
			break;

		case EHorizontalAlignment::HAlign_Right:
			SetBoxHorizontalAlignment(EHorizontalAlignment::HAlign_Left);
			break;

		default:
			// Do nothing
			break;
		}
	}

	if (ConstrainedOffset.Y > MidPoint.Y)
	{
		ConstrainedOffset.Y = FMath::Max(AvailableSpace.Y - ConstrainedOffset.Y, DraggableBoxDetail::DraggableBorderArea); // Circle value around

		switch (VerticalAlignment)
		{
		case EVerticalAlignment::VAlign_Top:
			SetBoxVerticalAlignment(EVerticalAlignment::VAlign_Bottom);
			break;

		case EVerticalAlignment::VAlign_Bottom:
			SetBoxVerticalAlignment(EVerticalAlignment::VAlign_Top);
			break;

		default:
			// Do nothing
			break;
		}
	}

	return ConstrainedOffset;
}

FToolWidget_DragBoxPosition SDraggableBoxOverlay::GetDragBoxPosition() const
{
	return FToolWidget_DragBoxPosition{ GetBoxAlignmentOffset(), HorizontalAlignment, VerticalAlignment };
}

void SDraggableBoxOverlay::SetDragBoxPosition(const FToolWidget_DragBoxPosition& InWidgetPosition)
{
	SetBoxHorizontalAlignment(InWidgetPosition.HAlign);
	SetBoxVerticalAlignment(InWidgetPosition.VAlign);

	// Do not recompute the anchors. Suppose the viewport is now much smaller than it was when InWidgetPosition was saved.
	// If the user now increases the size of the viewport, the widget should stay anchored to the same corner as when it was saved.
	constexpr bool bRecomputeAnchors = false;
	SetBoxAlignmentOffset(InWidgetPosition.RelativeOffset, bRecomputeAnchors);
}
	
void SDraggableBoxOverlay::RestoreFromDragBoxPosition(const FToolWidget_DragBoxPosition& InWidgetPosition)
{
	SetDragBoxPosition(InWidgetPosition);
}

TOptional<float> SDraggableBoxOverlay::GetWidthOverride() const
{
	return WidthOverride;
}

void SDraggableBoxOverlay::SetWidthOverride(const TOptional<float>& NewWidth)
{
	WidthOverride = NewWidth;
	if (WidthOverride.IsSet())
	{
		TOptional<float> MinBoxWidth = MinimumBoxWidth.Get();
		if (MinBoxWidth.IsSet())
		{
			WidthOverride = FMath::Max(WidthOverride.GetValue(), MinBoxWidth.GetValue());
		}
		TOptional<float> MaxBoxWidth = MaximumBoxWidth.Get();
		if (MaxBoxWidth.IsSet())
		{
			WidthOverride = FMath::Min(WidthOverride.GetValue(), MaxBoxWidth.GetValue());
		}
	}
}

TOptional<float> SDraggableBoxOverlay::GetHeightOverride() const
{
	return HeightOverride;
}

void SDraggableBoxOverlay::SetHeightOverride(const TOptional<float>& NewHeight)
{
	HeightOverride = NewHeight;
	if (HeightOverride.IsSet())
	{
		TOptional<float> MinBoxHeight = MinimumBoxHeight.Get();
		if (MinBoxHeight.IsSet())
		{
			HeightOverride = FMath::Max(HeightOverride.GetValue(), MinBoxHeight.GetValue());
		}
		TOptional<float> MaxBoxHeight = MaximumBoxHeight.Get();
		if (MaxBoxHeight.IsSet())
		{
			HeightOverride = FMath::Min(HeightOverride.GetValue(), MaxBoxHeight.GetValue());
		}
	}
}
	
TOptional<float> SDraggableBoxOverlay::GetMinimumBoxWidth() const
{
	return MinimumBoxWidth.Get();
}

TOptional<float> SDraggableBoxOverlay::GetMaximumBoxWidth() const
{
	return MaximumBoxWidth.Get();
}
	
TOptional<float> SDraggableBoxOverlay::GetMinimumBoxHeight() const
{
	return MinimumBoxHeight.Get();
}

TOptional<float> SDraggableBoxOverlay::GetMaximumBoxHeight() const
{
	return MaximumBoxHeight.Get();
}
}
