// Copyright Epic Games, Inc. All Rights Reserved.

#include "SActionButton.h"

#include "Styling/StyleColors.h"
#include "ToolWidgetsStyle.h"
#include "ToolWidgetsUtilitiesPrivate.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"

namespace UE::ToolWidgets::Private
{
	inline FName ActionButtonTypeToStyleName(const EActionButtonType ActionButtonType)
	{
		static TMap<EActionButtonType, const FName> Lookup = {
			{ EActionButtonType::Default, TEXT("ActionButton") },
			{ EActionButtonType::Simple, TEXT("SimpleComboButton") },
			{ EActionButtonType::Primary, TEXT("PrimaryButton") },
			{ EActionButtonType::Positive, TEXT("PositiveActionButton") },
			{ EActionButtonType::Warning, TEXT("NegativeActionButton.Warning") },
			{ EActionButtonType::Error, TEXT("NegativeActionButton.Error") },
		};

		if (const FName* FoundName = Lookup.Find(ActionButtonType))
		{
			return *FoundName;
		}

		return NAME_None;
	}
}

void SActionButton::Construct(const FArguments& InArgs)
{
	// Get overridden or default
	ActionButtonType = InArgs._ActionButtonType.Get(EActionButtonType::Default);

	// If style not explicitly set, derive from ActionButtonType
	if (!InArgs._ActionButtonStyle)
	{
		const FName StyleName = UE::ToolWidgets::Private::ActionButtonTypeToStyleName(ActionButtonType.Get());
		check(StyleName != NAME_None);

		ActionButtonStyle = &UE::ToolWidgets::FToolWidgetsStyle::Get().GetWidgetStyle<FActionButtonStyle>(StyleName);
	}
	else
	{
		ActionButtonStyle = InArgs._ActionButtonStyle;

		// ActionButtonStyle was specified, but ActionButtonType was not, so derive one from the other
		if (!InArgs._ActionButtonType.IsSet())
		{
			ActionButtonType = ActionButtonStyle->GetActionButtonType();
		}
	}

	ButtonStyle = InArgs._ButtonStyle ? InArgs._ButtonStyle : &ActionButtonStyle->ButtonStyle;
	IconButtonStyle = InArgs._IconButtonStyle ? InArgs._IconButtonStyle : &ActionButtonStyle->GetIconButtonStyle();
	ComboButtonStyle = InArgs._ComboButtonStyle ? InArgs._ComboButtonStyle : &ActionButtonStyle->ComboButtonStyle;
	TextBlockStyle = InArgs._TextBlockStyle ? InArgs._TextBlockStyle : &ActionButtonStyle->TextBlockStyle;

	const EHorizontalAlignment HorizontalContentAlignment =
		InArgs._HorizontalContentAlignment.IsSet()
		? InArgs._HorizontalContentAlignment.Get(EHorizontalAlignment::HAlign_Center)
		: static_cast<EHorizontalAlignment>(ActionButtonStyle->HorizontalContentAlignment);

	const bool bHasDownArrow = InArgs._HasDownArrow.IsSet() ? InArgs._HasDownArrow.Get(false) : ActionButtonStyle->bHasDownArrow;

	// Check for widget level override, then style override, otherwise unset
	const TAttribute<const FSlateBrush*> Icon =
		InArgs._Icon.IsSet()
		? InArgs._Icon
		: ActionButtonStyle->IconBrush.IsSet()
		? &ActionButtonStyle->IconBrush.GetValue()
		: TAttribute<const FSlateBrush*>();

	const bool bCanEverHaveIcon = Icon.IsBound() || Icon.Get(nullptr) != nullptr;

	const FButtonStyle* SelectedButtonStyle = bCanEverHaveIcon ? IconButtonStyle : ButtonStyle;

	// Check for widget level override, then style override, otherwise get from ActionButtonType
	const TAttribute<FSlateColor> IconColorAndOpacity =
		InArgs._IconColorAndOpacity.IsSet()
		? InArgs._IconColorAndOpacity.Get()
		: ActionButtonStyle->IconColorAndOpacity.IsSet()
		? TAttribute<FSlateColor>(ActionButtonStyle->IconColorAndOpacity.GetValue())
		: TAttribute<FSlateColor>::Create(TAttribute<FSlateColor>::FGetter::CreateSP(this, &SActionButton::GetIconColorAndOpacity));

	TAttribute<FText> Text = InArgs._Text;

	const TSharedRef<SWidget> ButtonContent =
		UE::ToolWidgets::Private::ActionButton::MakeButtonContent(
			Icon,
			IconColorAndOpacity,
			Text,
			TextBlockStyle);

	// Treated as a ComboButton if OnClicked is not bound
	const bool bIsComboButton = !InArgs._OnClicked.IsBound();

	if (bIsComboButton)
	{
		const TAttribute<FMargin> ComboButtonContentPadding =
			InArgs._ButtonContentPadding.IsSet()
			? InArgs._ButtonContentPadding.Get()
			: ActionButtonStyle->GetComboButtonContentPadding();

		ChildSlot
		[
			SAssignNew(ComboButton, SComboButton)
			.HasDownArrow(bHasDownArrow)
			.ContentPadding(ComboButtonContentPadding)
			.ButtonStyle(SelectedButtonStyle)
			.ComboButtonStyle(ComboButtonStyle)
			.IsEnabled(InArgs._IsEnabled)
			.ToolTipText(InArgs._ToolTipText)
			.HAlign(HorizontalContentAlignment)
			.VAlign(VAlign_Center)
			.ButtonContent()
			[
				ButtonContent
			]
			.MenuContent()
			[
				InArgs._MenuContent.Widget
			]
			.OnGetMenuContent(InArgs._OnGetMenuContent)
			.OnMenuOpenChanged(InArgs._OnMenuOpenChanged)
			.OnComboBoxOpened(InArgs._OnComboBoxOpened)
		];
	}
	else
	{
		const TAttribute<FMargin> ButtonContentPadding =
			InArgs._ButtonContentPadding.IsSet()
			? InArgs._ButtonContentPadding.Get()
			: ActionButtonStyle->GetButtonContentPadding();

		ChildSlot
		[
			SAssignNew(Button, SButton)
			.ContentPadding(ButtonContentPadding)
			.ButtonStyle(SelectedButtonStyle)
			.IsEnabled(InArgs._IsEnabled)
			.ToolTipText(InArgs._ToolTipText)
			.HAlign(HorizontalContentAlignment)
			.VAlign(VAlign_Center)
			.NormalPaddingOverride(UE::ToolWidgets::Private::ActionButton::MakeIconPaddingOverride(Icon, SelectedButtonStyle->NormalPadding, ActionButtonStyle->IconNormalPadding))
			.PressedPaddingOverride(UE::ToolWidgets::Private::ActionButton::MakeIconPaddingOverride(Icon, SelectedButtonStyle->PressedPadding, ActionButtonStyle->IconPressedPadding))
			.OnClicked(InArgs._OnClicked)
			[
				ButtonContent
			]
		];
	}
}

void SActionButton::SetMenuContentWidgetToFocus(TWeakPtr<SWidget> InWidget)
{
	check(ComboButton.IsValid());
	ComboButton->SetMenuContentWidgetToFocus(InWidget);
}

void SActionButton::SetIsMenuOpen(bool bInIsOpen, bool bInIsFocused)
{
	check(ComboButton.IsValid());
	ComboButton->SetIsOpen(bInIsOpen, bInIsFocused);
}

FSlateColor SActionButton::GetIconColorAndOpacity() const
{
	switch (ActionButtonType.Get(EActionButtonType::Default))
	{
	case EActionButtonType::Default:
	case EActionButtonType::Simple:
	default:
		return FSlateColor::UseForeground();

	case EActionButtonType::Positive:
		return FStyleColors::AccentGreen;

	case EActionButtonType::Warning:
		return FStyleColors::Warning;

	case EActionButtonType::Error:
		return FStyleColors::Error;
	}
}
