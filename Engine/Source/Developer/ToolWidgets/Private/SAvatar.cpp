// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAvatar.h"
#include "Containers/StringConv.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBox.h"
#include "Styling/CoreStyle.h"
#include "Fonts/FontMeasure.h"
#include "Math/RandomStream.h"

void SAvatar::Construct(const FArguments& InArgs)
{
	Identifier = InArgs._Identifier;
	Description = InArgs._Description;

	bShowInitial = InArgs._ShowInitial;

	BackgroundColor = ComputeBackgroundColor();
	ForegroundColor = FColor::White;

	ChildSlot
	[
		SNew(SBox)
		.MinDesiredWidth(InArgs._WidthOverride)
		.MinDesiredHeight(InArgs._HeightOverride)
		.Padding(0.0f)
	];
}

int32 SAvatar::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const float Radius = FMath::Max(AllottedGeometry.GetLocalSize().X, AllottedGeometry.GetLocalSize().Y) * 0.5f;

	// Draw background circle.

	const FSlateBrush* CircleBrush = FCoreStyle::Get().GetBrush("Icons.FilledCircle");

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId++,
		AllottedGeometry.ToPaintGeometry(),
		CircleBrush,
		bParentEnabled && IsEnabled() ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
		BackgroundColor
	);

	// Draw foreground text.

	if (Description.Len() > 0 && bShowInitial)
	{
		FSlateFontInfo FontInfo = FCoreStyle::GetDefaultFontStyle("Bold", Radius);

		const FString Initial = Description.Left(1).ToUpper();
		const FString Text = FChar::IsAlnum(Initial[0]) ? Initial : TEXT("?");
		const FVector2D TextSize = FSlateApplication::Get().GetRenderer()->GetFontMeasureService()->Measure(Text, FontInfo);
		const FVector2D TextOffset(Radius - TextSize.X / 2, Radius - TextSize.Y / 2);

		FSlateDrawElement::MakeText(
			OutDrawElements,
			LayerId++,
			AllottedGeometry.ToPaintGeometry(AllottedGeometry.GetLocalSize(), FSlateLayoutTransform(TextOffset)),
			Text,
			FontInfo,
			bParentEnabled && IsEnabled() ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			ForegroundColor
		);
	}

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled && IsEnabled());
}

uint32 SAvatar::Hash() const
{
	// Warning: changing the hash computation could create discrepancies with other Epic owned systems' avatar
	// appearance.
	uint32 HashValue = 5381;
	using FStringUtf8 = TStringConversion<TStringConvert<FString::ElementType, UTF8CHAR>>;
	const FStringUtf8 IdentifierUtf8 = StringCast<UTF8CHAR>(*Identifier);
	const FUtf8StringView IdentifierUtf8View(IdentifierUtf8.Get(), IdentifierUtf8.Length());
	for (const UTF8CHAR Byte : IdentifierUtf8View)
	{
		HashValue = ((HashValue << 5) + HashValue) + Byte;
	}
	return HashValue;
}

FColor SAvatar::ComputeBackgroundColor() const
{
	const uint64 Value = Hash();
	const uint8 H = ((Value >> 0) & 0xFF) ^ ((Value >> 8) & 0xFF) ^ ((Value >> 16) & 0xFF) ^ ((Value >> 24) & 0xFF);
	const uint8 S = 128 + ((Value >> 8) & 0x7F);
	const uint8 V = 128 + ((Value >> 16) & 0x7F) / 2;
	return FLinearColor::MakeFromHSV8(H, S, V).ToFColor(false);
}