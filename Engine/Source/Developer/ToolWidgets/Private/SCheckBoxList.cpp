// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCheckBoxList.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SCheckBoxList"

namespace CheckBoxList
{
	const FName ColumnID_CheckBox("CheckBox");
	const FName ColumnID_Item("Item");

	const float CheckBoxColumnWidth = 23.0f;

	struct FItemPair
	{
		TSharedRef<SWidget> Widget;
		bool bIsChecked;
		FItemPair(TSharedRef<SWidget> InWidget, bool bInChecked)
			: Widget(InWidget), bIsChecked(bInChecked)
		{ }
	};

	class SItemPair : public SMultiColumnTableRow<TSharedRef<FItemPair>>
	{
	public:
		SLATE_BEGIN_ARGS(SItemPair) { }
			SLATE_STYLE_ARGUMENT(FCheckBoxStyle, CheckBoxStyle)
			SLATE_ARGUMENT(FSimpleDelegate, OnCheckUpdated)
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner, TSharedRef<FItemPair> InItem)
		{
			CheckBoxStyle = InArgs._CheckBoxStyle;
			OnCheckUpdated = InArgs._OnCheckUpdated;
			Item = InItem;
			FSuperRowType::Construct(FTableRowArgs(), InOwner);
		}

	public:
		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (ColumnName == ColumnID_CheckBox)
			{
				return SNew(SCheckBox)
				.Style(CheckBoxStyle)
				.IsChecked(this, &SItemPair::GetToggleSelectedState)
				.OnCheckStateChanged(this, &SItemPair::OnToggleSelectedCheckBox);
			}
			else if (ColumnName == ColumnID_Item)
			{
				return Item->Widget;
			}
			check(false);
			return SNew(SCheckBox);
		}

		ECheckBoxState GetToggleSelectedState() const
		{
			return Item->bIsChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		void OnToggleSelectedCheckBox(ECheckBoxState InNewState)
		{
			Item->bIsChecked = InNewState == ECheckBoxState::Checked;
			OnCheckUpdated.ExecuteIfBound();
		}

		TSharedPtr<FItemPair> Item;
		const FCheckBoxStyle* CheckBoxStyle;
		FSimpleDelegate OnCheckUpdated;
	};
}

void SCheckBoxList::Construct(const FArguments& InArgs)
{
	Construct(InArgs, TArray<TSharedRef<SWidget>>(), false);
}

void SCheckBoxList::Construct(const FArguments& InArgs, const TArray<FText>& InItems, bool bIsChecked)
{
	TArray<TSharedRef<SWidget>> Widgets;
	Widgets.Reserve(InItems.Num());
	for (const FText& Text : InItems)
	{
		Widgets.Add(SNew(STextBlock).Text(Text));
	}
	Construct(InArgs, Widgets, bIsChecked);
}

void SCheckBoxList::Construct(const FArguments& InArgs, const TArray<TSharedRef<SWidget>>& InItems, bool bIsChecked)
{
	CheckBoxStyle = InArgs._CheckBoxStyle;

	Items.Reserve(InItems.Num());
	for (TSharedRef<SWidget> Widget : InItems)
	{
		Items.Add(MakeShared<CheckBoxList::FItemPair>(Widget, bIsChecked));
	}

	bool bShowHeaderCheckbox = InArgs._IncludeGlobalCheckBoxInHeaderRow;
	OnItemCheckStateChanged = InArgs._OnItemCheckStateChanged;

	TSharedRef<SHeaderRow> HeaderRowWidget = SNew(SHeaderRow)
		+ SHeaderRow::Column(CheckBoxList::ColumnID_CheckBox)
		[
			SNew(SCheckBox)
			.Style(InArgs._CheckBoxStyle)
			.IsChecked(this, &SCheckBoxList::GetToggleSelectedState)
			.OnCheckStateChanged(this, &SCheckBoxList::OnToggleSelectedCheckBox)
			.Visibility_Lambda([bShowHeaderCheckbox] { return bShowHeaderCheckbox ? EVisibility::Visible : EVisibility::Hidden; })
		]
		.FixedWidth(CheckBoxList::CheckBoxColumnWidth)
		+ SHeaderRow::Column(CheckBoxList::ColumnID_Item)
		.DefaultLabel(InArgs._ItemHeaderLabel)
		.FillWidth(1.0f);

	ChildSlot
	[
		SAssignNew(ListView, SListView<TSharedRef<CheckBoxList::FItemPair>>)
		.ListItemsSource(&Items)
		.OnGenerateRow(this, &SCheckBoxList::HandleGenerateRow)
		.HeaderRow(HeaderRowWidget)
		.SelectionMode(ESelectionMode::None)
	];

	// Set the Global CheckBox to be correct
	UpdateAllChecked();
}

int32 SCheckBoxList::AddItem(const FText& Text, bool bIsChecked)
{
	int32 ReturnValue = Items.Add(MakeShared<CheckBoxList::FItemPair>(SNew(STextBlock).Text(Text), bIsChecked));
	UpdateAllChecked();	
	ListView->RebuildList();
	return ReturnValue;
}

int32 SCheckBoxList::AddItem(TSharedRef<SWidget> Widget, bool bIsChecked)
{
	int32 ReturnValue = Items.Add(MakeShared<CheckBoxList::FItemPair>(Widget, bIsChecked));
	UpdateAllChecked();	
	ListView->RebuildList();
	return ReturnValue;
}

void SCheckBoxList::RemoveAll()
{
	Items.Reset();
	UpdateAllChecked();
	ListView->RebuildList();
}

void SCheckBoxList::RemoveItem(int32 Index)
{
	if (Items.IsValidIndex(Index))
	{
		Items.RemoveAt(Index);
		UpdateAllChecked();
		ListView->RebuildList();
	}
}

bool SCheckBoxList::IsItemChecked(int32 Index) const
{
	return Items.IsValidIndex(Index) ? Items[Index]->bIsChecked : false;
}

TArray<bool> SCheckBoxList::GetValues() const
{
	TArray<bool> Values;
	for (const TSharedRef<CheckBoxList::FItemPair>& Item : Items)
	{
		Values.Add(Item->bIsChecked);
	}
	return Values;
}

int32 SCheckBoxList::GetNumCheckboxes() const
{
	return Items.Num();
}

void SCheckBoxList::UpdateAllChecked()
{
	bool bContainsTrue = Items.ContainsByPredicate([](const TSharedRef<CheckBoxList::FItemPair>& Item) { return Item->bIsChecked; });
	bool bContainsFalse = Items.ContainsByPredicate([](const TSharedRef<CheckBoxList::FItemPair>& Item) { return !Item->bIsChecked; });
	bAllCheckedState = bContainsTrue && bContainsFalse ? ECheckBoxState::Undetermined : (bContainsTrue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked);
}

ECheckBoxState SCheckBoxList::GetToggleSelectedState() const
{
	return bAllCheckedState;
}

void SCheckBoxList::OnToggleSelectedCheckBox(ECheckBoxState InNewState)
{
	bool bNewValue = InNewState == ECheckBoxState::Checked;
	for (TSharedRef<CheckBoxList::FItemPair>& Item : Items)
	{
		Item->bIsChecked = bNewValue;
	}
	bAllCheckedState = bNewValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	
	OnItemCheckStateChanged.ExecuteIfBound(-1);
}

void SCheckBoxList::OnItemCheckBox(TSharedRef<CheckBoxList::FItemPair> InItem)
{
	UpdateAllChecked();
	OnItemCheckStateChanged.ExecuteIfBound(Items.IndexOfByKey(InItem));
}

TSharedRef<ITableRow> SCheckBoxList::HandleGenerateRow(TSharedRef<CheckBoxList::FItemPair> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(CheckBoxList::SItemPair, OwnerTable, InItem)
		.CheckBoxStyle(CheckBoxStyle)
		.OnCheckUpdated(FSimpleDelegate::CreateLambda([this, InItem]() { OnItemCheckBox(InItem); }));
}

#undef LOCTEXT_NAMESPACE
