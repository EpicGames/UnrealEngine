// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNegativeActionButton.h"

#include "SActionButton.h"
#include "ToolWidgetsStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

namespace UE::ToolWidgets::Private
{
	EActionButtonType ActionButtonStyleToActionButtonType(const EActionButtonStyle InStyle)
	{
		static TMap<EActionButtonStyle, EActionButtonType> Lookup = {
			{ EActionButtonStyle::Warning, EActionButtonType::Warning },
			{ EActionButtonStyle::Error, EActionButtonType::Error }
		};

		if (const EActionButtonType* FoundEnum = Lookup.Find(InStyle))
		{
			return *FoundEnum;
		}

		return EActionButtonType::Default;
	}
}

void SNegativeActionButton::Construct(const FArguments& InArgs)
{
	// @note: Delegate-bound ActionButtonStyle is not supported, it should be provided at construction.
	const EActionButtonType ActionButtonType = UE::ToolWidgets::Private::ActionButtonStyleToActionButtonType(InArgs._ActionButtonStyle.Get(EActionButtonStyle::Warning));

	ActionButton =
		SNew(SActionButton)
		.ActionButtonStyle(
			InArgs._ActionButtonStyle.Get(EActionButtonStyle::Warning) == EActionButtonStyle::Warning
			? &UE::ToolWidgets::FToolWidgetsStyle::Get().GetWidgetStyle<FActionButtonStyle>("NegativeActionButton.Warning")
			: &UE::ToolWidgets::FToolWidgetsStyle::Get().GetWidgetStyle<FActionButtonStyle>("NegativeActionButton.Error"))
		.ActionButtonType(ActionButtonType)
		.Text(InArgs._Text)
		.Icon(InArgs._Icon)
		.OnClicked(InArgs._OnClicked)
		.OnGetMenuContent(InArgs._OnGetMenuContent)
		.OnComboBoxOpened(InArgs._OnComboBoxOpened)
		.OnMenuOpenChanged(InArgs._OnMenuOpenChanged)
		.MenuContent()
		[
			InArgs._MenuContent.Widget
		];

	ChildSlot
	[
		ActionButton.ToSharedRef()
	];
}

void SNegativeActionButton::SetMenuContentWidgetToFocus(TWeakPtr<SWidget> InWidget)
{
	check(ActionButton.IsValid());

	ActionButton->SetMenuContentWidgetToFocus(InWidget);
}

void SNegativeActionButton::SetIsMenuOpen(bool bInIsOpen, bool bInIsFocused)
{
	check(ActionButton.IsValid());

	ActionButton->SetIsMenuOpen(bInIsOpen, bInIsFocused);
}
