// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPositiveActionButton.h"

#include "SActionButton.h"
#include "ToolWidgetsStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

void SPositiveActionButton::Construct(const FArguments& InArgs)
{
	check(InArgs._Icon.IsSet());

	ActionButton =
		SNew(SActionButton)
		.ActionButtonStyle(&UE::ToolWidgets::FToolWidgetsStyle::Get().GetWidgetStyle<FActionButtonStyle>("PositiveActionButton"))
		.Text(InArgs._Text)
		.Icon(InArgs._Icon)
		.OnClicked(InArgs._OnClicked)
		.OnGetMenuContent(InArgs._OnGetMenuContent)
		.OnComboBoxOpened(InArgs._OnComboBoxOpened)
		.OnMenuOpenChanged(InArgs._OnMenuOpenChanged)
		.MenuContent()
		[
			InArgs._MenuContent.Widget
		];

	ChildSlot
	[
		ActionButton.ToSharedRef()
	];
}

void SPositiveActionButton::SetMenuContentWidgetToFocus(TWeakPtr<SWidget> InWidget)
{
	check(ActionButton.IsValid());

	ActionButton->SetMenuContentWidgetToFocus(InWidget);
}

void SPositiveActionButton::SetIsMenuOpen(bool bInIsOpen, bool bInIsFocused)
{
	check(ActionButton.IsValid());

	ActionButton->SetIsMenuOpen(bInIsOpen, bInIsFocused);
}
