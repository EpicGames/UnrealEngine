// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPrimaryButton.h"

#include "ToolWidgetsSlateTypes.h"
#include "ToolWidgetsStyle.h"
#include "ToolWidgetsUtilitiesPrivate.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

void SPrimaryButton::Construct(const FArguments& InArgs)
{
	const FActionButtonStyle* ActionButtonStyle = &UE::ToolWidgets::FToolWidgetsStyle::Get().GetWidgetStyle<FActionButtonStyle>("PrimaryButton");

	// Check for widget level override, then style override, otherwise unset
	const TAttribute<const FSlateBrush*> Icon = InArgs._Icon.IsSet()
		? InArgs._Icon
		: ActionButtonStyle->IconBrush.IsSet()
		? &ActionButtonStyle->IconBrush.GetValue()
		: nullptr;

	const bool bCanEverHaveIcon = Icon.IsBound() || Icon.Get(nullptr) != nullptr;

	const FButtonStyle& SelectedButtonStyle = bCanEverHaveIcon ? ActionButtonStyle->GetIconButtonStyle() : ActionButtonStyle->ButtonStyle;

	// Empty/default args will resolve from the ActionButtonStyle
	const TSharedRef<SWidget> ButtonContent =
		UE::ToolWidgets::Private::ActionButton::MakeButtonContent(
			ActionButtonStyle,
			Icon,
			{},
			InArgs._Text,
			{});

	SButton::Construct(
		SButton::FArguments()
		.ContentPadding(ActionButtonStyle->GetButtonContentPadding())
		.ButtonStyle(&SelectedButtonStyle)
		.IsEnabled(InArgs._IsEnabled)
		.ToolTipText(InArgs._ToolTipText)
		.HAlign(static_cast<EHorizontalAlignment>(ActionButtonStyle->HorizontalContentAlignment))
		.VAlign(VAlign_Center)
		.NormalPaddingOverride(UE::ToolWidgets::Private::ActionButton::MakeIconPaddingOverride(Icon, SelectedButtonStyle.NormalPadding, ActionButtonStyle->IconNormalPadding))
		.PressedPaddingOverride(UE::ToolWidgets::Private::ActionButton::MakeIconPaddingOverride(Icon, SelectedButtonStyle.PressedPadding, ActionButtonStyle->IconPressedPadding))
		.OnClicked(InArgs._OnClicked)
		[
			ButtonContent
		]);
}
