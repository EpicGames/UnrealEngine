// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRichTextWarningOrErrorBox.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"

void SRichTextWarningOrErrorBox::Construct(const FArguments& InArgs)
{
	MessageStyle = InArgs._MessageStyle;

	SBorder::Construct(SBorder::FArguments()
		.Padding(InArgs._Padding)
		.ForegroundColor(FAppStyle::Get().GetSlateColor("Colors.White"))
		.BorderImage_Lambda([this]() { return MessageStyle.Get() == EMessageStyle::Warning ? FAppStyle::Get().GetBrush("RoundedWarning") : FAppStyle::Get().GetBrush("RoundedError"); })
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			.AutoWidth()
			.Padding(FMargin(0.0f, 0.0f, 16.0f, 0.0f))
			[
				SNew(SImage)
				.DesiredSizeOverride(InArgs._IconSize)
				.Image_Lambda([this]() { return MessageStyle.Get() == EMessageStyle::Warning ? FAppStyle::Get().GetBrush("Icons.WarningWithColor") : FAppStyle::Get().GetBrush("Icons.ErrorWithColor"); })
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(SRichTextBlock)
				.DecoratorStyleSet(&FAppStyle::Get())
				.Text(InArgs._Message)
				.AutoWrapText(true)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			[
				InArgs._Content.Widget
			]

		]);
}

