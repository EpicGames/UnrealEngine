// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSearchToggleButton.h"

#include "Framework/Application/SlateApplication.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Styling/SlateColor.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"

void SSearchToggleButton::Construct(const FArguments& InArgs, TSharedRef<SSearchBox> SearchBox)
{
	bIsExpanded = false;
	OnSearchBoxShown = InArgs._OnSearchBoxShown;

	SearchStyle = InArgs._Style;

	SearchBox->SetVisibility(TAttribute<EVisibility>::CreateSP(this, &SSearchToggleButton::GetSearchBoxVisibility));
	SearchBoxPtr = SearchBox;

	ChildSlot
	[
		SNew(SCheckBox)
		.IsChecked(this, &SSearchToggleButton::GetToggleButtonState)
		.OnCheckStateChanged(this, &SSearchToggleButton::OnToggleButtonStateChanged)
		.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
		.Padding(4.0f)
		.ToolTipText(NSLOCTEXT("ExpandableSearchArea", "ExpandCollapseSearchButton", "Expands or collapses the search text box"))
		[
			SNew(SImage)
			.Image(&SearchStyle->GlassImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
	];
}

void SSearchToggleButton::SetExpanded(bool bInExpanded)
{
	bIsExpanded = bInExpanded;

	if (TSharedPtr<SSearchBox> SearchBox = SearchBoxPtr.Pin())
	{
		if (bIsExpanded)
		{
			OnSearchBoxShown.ExecuteIfBound();

			// Focus the search box when it's shown
			FSlateApplication::Get().SetUserFocus(FSlateApplication::Get().GetUserIndexForKeyboard(), SearchBox, EFocusCause::SetDirectly);
		}
		else
		{
			// Clear the search box when it's hidden
			SearchBox->SetText(FText::GetEmpty());
		}
	}
}

void SSearchToggleButton::OnToggleButtonStateChanged(ECheckBoxState CheckBoxState)
{
	SetExpanded(CheckBoxState == ECheckBoxState::Checked);
}

ECheckBoxState SSearchToggleButton::GetToggleButtonState() const
{
	return bIsExpanded ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

EVisibility SSearchToggleButton::GetSearchBoxVisibility() const
{
	return bIsExpanded ? EVisibility::Visible : EVisibility::Collapsed;
}
