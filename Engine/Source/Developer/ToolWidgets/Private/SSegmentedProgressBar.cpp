// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSegmentedProgressBar.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Styling/StyleColors.h"
#include "Layout/LayoutUtils.h"

#define LOCTEXT_NAMESPACE "SSegmentedProgressBar"

SSegmentedProgressBar::FScopedWidgetSlotArguments SSegmentedProgressBar::InsertSlot(int32 Index, bool bRebuildChildren)
{
	if (bRebuildChildren)
	{
		TWeakPtr<SSegmentedProgressBar> AsWeak = SharedThis(this);
		return FScopedWidgetSlotArguments { MakeUnique<FSlot>(), this->Children, Index, [AsWeak](const FSlot*, int32)
		{
			if (TSharedPtr<SSegmentedProgressBar> SharedThis = AsWeak.Pin())
			{
				SharedThis->RebuildChildren();
			}
		}};
	}
	else
	{
		return FScopedWidgetSlotArguments(MakeUnique<FSlot>(), this->Children, Index);
	}
}



void SSegmentedProgressBar::Construct( const FArguments& InArgs )
{
	Style = InArgs._Style;
	bAutoPromoteNextPending = InArgs._bAutoPromoteNextPending;
	ThrobberAnimation = FCurveSequence(0.0f, 1.0f);
	ThrobberAnimation.Play(AsShared(), true);

	Children.AddSlots(MoveTemp(const_cast<TArray<typename FSlot::FSlotArguments>&>(InArgs._Slots)));	
	RebuildChildren();
}



void SSegmentedProgressBar::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	CacheChildStates();
}



void SSegmentedProgressBar::CacheChildStates()
{
	CachedChildStates.Reset();
	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ChildIndex++)
	{
		EState ChildState = Children[ChildIndex].State.Get();

		// show us as busy as soon as the previous task is completed - just looks nicer
		if (bAutoPromoteNextPending && ChildState == EState::Pending && (ChildIndex == 0 || CachedChildStates[ChildIndex-1] == EState::Completed))
		{
			ChildState = EState::Busy;
		}

		CachedChildStates.Add( ChildState );
	}

	if (CachedChildStates.Num() == 0 )
	{
		CachedOverallState = EState::None;
	}
	else
	{
		CachedOverallState = CachedChildStates.Last();
	}
}



SSegmentedProgressBar::EState SSegmentedProgressBar::GetChildState( int32 ChildIndex ) const
{
	if (CachedChildStates.IsValidIndex(ChildIndex))
	{
		return CachedChildStates[ChildIndex];
	}
	
	return EState::None;
}



void SSegmentedProgressBar::RebuildChildren()
{
	CacheChildStates();

	TSharedPtr<SHorizontalBox> CircleRow;
	
	TSharedPtr<SVerticalBox> Container;
	ChildSlot
	[ 
		SAssignNew(Container, SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(CircleRow, SHorizontalBox)
		]
	];

	// Should we add optional labels?
	TSharedPtr<SHorizontalBox> LabelRow;

	bool bHasLabels = false;
	for (int32 i = 0; i < Children.Num(); ++i)
	{
		if (Children[i].LabelContent.IsValid())
		{
			bHasLabels = true;
			break;
		}
	}
	if (bHasLabels)
	{
		Container->AddSlot()
		.AutoHeight()
		.Padding(0.0f, 4.0f, 0.0f, 0.0f)
		[
			SAssignNew(LabelRow, SHorizontalBox)
		];
	}

	const int32 NumChildren = Children.Num();
	for ( int32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++ )
	{
		TSharedRef<SWidget> Child = Children[ChildIndex].GetWidget();
		FSlot* ChildSlotPtr = &Children[ChildIndex];

		if (Child == SNullWidget::NullWidget)
		{
			Child = ConstructChild(*ChildSlotPtr, ChildIndex);
		}

		// separator bar between previous and current
		if (ChildIndex > 0)
		{
			AddChildSeparatorBar(CircleRow, ChildIndex);
			if (LabelRow)
			{
				LabelRow->AddSlot()
				.FillWidth(1)
				[ 
					SNullWidget::NullWidget 
				];
			}
		}

		// task item
		CircleRow->AddSlot()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Center)
		.AutoWidth()
		[
			Child
		];

		if (LabelRow)
		{
			LabelRow->AddSlot()
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(36.0f)
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Fill)
					.Padding(-1000.0f, 0.0f)
					[
						ChildSlotPtr->LabelContent.IsValid()
							? ChildSlotPtr->LabelContent.ToSharedRef()
							: SNullWidget::NullWidget
					]
				]
			];
		}
	}
}



void SSegmentedProgressBar::AddChildSeparatorBar( TSharedPtr<SHorizontalBox> SlotBox, int32 ChildIndex ) const
{
	SlotBox->AddSlot()
	.VAlign(VAlign_Top)
	.HAlign(HAlign_Fill)
	.FillWidth(1)
	.Padding(0.0f, (36.0f - LineSize) / 2.0f, 0.0f, 0.0f)
	[
		SNew(SColorBlock)
		.Color( this, &SSegmentedProgressBar::GetSeparatorBarColor, ChildIndex )
		.Size( FVector2D(32, LineSize) )
	];
}



TSharedRef<SWidget> SSegmentedProgressBar::ConstructChild( const FSlot& Slot, int32 ChildIndex ) const
{
	TSharedRef<SOverlay> CircleOverlay = SNew(SOverlay)
	.ToolTipText(Slot.ToolTipText)

	// full circle (only shown when fully complete or cancelled)
	+SOverlay::Slot()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Center)
	[
		SNew(SImage)
		.Image( Style.FullCircleBrush )
		.ColorAndOpacity( this, &SSegmentedProgressBar::GetCircleColor, ChildIndex)
		.Visibility( this, &SSegmentedProgressBar::GetFullCircleVisibility, ChildIndex )

	]

	// outer circle
	+SOverlay::Slot()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Center)
	[
		SNew(SImage)
		.DesiredSizeOverride(FVector2D(36,36))
		.Image( Style.OuterCircleBrush )
		.ColorAndOpacity( this, &SSegmentedProgressBar::GetCircleColor, ChildIndex )
		.Visibility( this, &SSegmentedProgressBar::GetOuterCircleVisibility, ChildIndex )
	]

	// outer busy circle
	+SOverlay::Slot()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Center)
	[
		SNew(SImage)
		.DesiredSizeOverride(FVector2D(36,36))
		.Image( Style.OuterCircleBusyBrush )
		.ColorAndOpacity( Style.BusyColor )
		.Visibility( this, &SSegmentedProgressBar::GetProgressCircleVisibility, ChildIndex )
		.RenderTransform( this, &SSegmentedProgressBar::GetProgressCircleTransform, ChildIndex)
		.RenderTransformPivot(FVector2D(.5f,.5f))
	]

	// task icon
	+SOverlay::Slot()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Center)
	[
		SNew(SImage)
		.Image(Slot.Image)
		.ColorAndOpacity(this, &SSegmentedProgressBar::GetIconColor, ChildIndex)
		.DesiredSizeOverride(FVector2D(16,16))
	]

	// task state overlay
	+SOverlay::Slot()
	.VAlign(VAlign_Bottom)
	.HAlign(HAlign_Right)
	[
		SNew(SImage)
		.Image(this, &SSegmentedProgressBar::GetOverlayIcon, ChildIndex)
		.Visibility( this, &SSegmentedProgressBar::GetOverlayVisibility, ChildIndex)
	]
	;

	return SNew(SVerticalBox)
	+ SVerticalBox::Slot()
	.AutoHeight()
	.HAlign(HAlign_Center)
	[
		CircleOverlay
	];
}


FLinearColor SSegmentedProgressBar::GetSeparatorBarColor( int32 ChildIndex ) const
{
	if (CachedOverallState == EState::Completed)
	{
		return Style.AllCompleteColor.GetSpecifiedColor();
	}

	EState State = GetChildState(ChildIndex);
	switch (State)
	{
		case EState::Busy:		return Style.BusyColor.GetSpecifiedColor();
		case EState::Canceled:  return Style.CanceledColor.GetSpecifiedColor();
		case EState::Skipped:   return Style.SkippedColor.GetSpecifiedColor();
		case EState::Completed: return Style.CompleteColor.GetSpecifiedColor(); // line indicates previous step succeeded in this case
		case EState::Failed:	return Style.CompleteColor.GetSpecifiedColor(); // failed can use complete's color
		case EState::Pending:	return Style.PendingColor.GetSpecifiedColor();
	}

	return FSlateColor::UseForeground().GetSpecifiedColor();
}

FSlateColor SSegmentedProgressBar::GetCircleColor( int32 ChildIndex ) const
{
	EState State = GetChildState(ChildIndex);

	// Skipped stays visually distinct from the rest of the flow at all times,
	// including when the overall flow has completed, because it never actually executed.
	if (State == EState::Skipped)
	{
		return Style.SkippedColor;
	}

	if (CachedOverallState == EState::Completed)
	{
		return Style.AllCompleteColor;
	}

	switch (State)
	{
		case EState::Busy:		return Style.PendingColor;
		case EState::Canceled:  return Style.CanceledColor;
		case EState::Completed:	return Style.CompleteColor;
		case EState::Failed:	return Style.ErrorColor;
		case EState::Pending:	return Style.PendingColor;
	}

	return FSlateColor::UseForeground();
}





FSlateColor SSegmentedProgressBar::GetIconColor( int32 ChildIndex ) const
{
	EState State = GetChildState(ChildIndex);

	// Skipped stays visually distinct from the rest of the flow at all times,
	// including when the overall flow has completed, because it never actually executed.
	if (State == EState::Skipped)
	{
		return FStyleColors::Hover2;
	}

	if (CachedOverallState == EState::Completed)
	{
		return FStyleColors::White;
	}

	if (State == EState::Canceled)
	{
		return FStyleColors::Hover2;
	}

	return FStyleColors::Foreground;
}


EVisibility SSegmentedProgressBar::GetFullCircleVisibility( int32 ChildIndex ) const
{
	EState ChildState = GetChildState(ChildIndex);
	if (CachedOverallState == EState::Completed || ChildState == EState::Canceled || ChildState == EState::Skipped)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}



TOptional<FSlateRenderTransform> SSegmentedProgressBar::GetProgressCircleTransform( int32 ChildIndex ) const
{
	if (GetChildState(ChildIndex) == EState::Busy)
	{
		const float DeltaAngle = ThrobberAnimation.GetLerp()*2*PI;
		return FSlateRenderTransform(FQuat2D(DeltaAngle));
	}

	FSlateRenderTransform Result;
	return Result;
}



EVisibility SSegmentedProgressBar::GetOuterCircleVisibility( int32 ChildIndex ) const
{
	EState ChildState = GetChildState(ChildIndex);
	if (CachedOverallState != EState::Completed && ChildState != EState::Canceled && ChildState != EState::Skipped)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}



EVisibility SSegmentedProgressBar::GetProgressCircleVisibility( int32 ChildIndex ) const
{
	if (CachedOverallState != EState::Completed && GetChildState(ChildIndex) == EState::Busy)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}




EVisibility SSegmentedProgressBar::GetOverlayVisibility( int32 ChildIndex ) const
{
	EState State = GetChildState(ChildIndex);
	if (State == EState::Completed || State == EState::Failed)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}



const FSlateBrush* SSegmentedProgressBar::GetOverlayIcon( int32 ChildIndex ) const
{
	if (CachedOverallState == EState::Completed)
	{
		return Style.BadgeAllCompleteBrush;
	}

	EState State = GetChildState(ChildIndex);
	switch (State)
	{
		case EState::Completed: return Style.BadgeSuccessBrush;
		case EState::Failed:	return Style.BadgeErrorBrush;
	}

	return FStyleDefaults::GetNoBrush();
}

#undef LOCTEXT_NAMESPACE