// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSegmentedTimelineView.h"
#include "Rendering/DrawElements.h"
#include "SSimpleTimeSlider.h"

#define LOCTEXT_NAMESPACE "SSegmentedTimelineView"

void SSegmentedTimelineView::Construct(const SSegmentedTimelineView::FArguments& InArgs)
{
	ViewRange = InArgs._ViewRange;
	FillColor = InArgs._FillColor;
	DesiredSize = InArgs._DesiredSize;
	SegmentData = InArgs._SegmentData;
}

int32 SSegmentedTimelineView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 NewLayer = PaintBlock(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	return FMath::Max(NewLayer, SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, NewLayer, InWidgetStyle, ShouldBeEnabled(bParentEnabled)));
}


FVector2D SSegmentedTimelineView::ComputeDesiredSize(float) const
{
	return UE::Slate::FDeprecateVector2DResult(DesiredSize.Get());
}

int32 SSegmentedTimelineView::PaintBlock(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// convert time range to from rewind debugger times to profiler times
	TRange<double> DebugTimeRange = ViewRange.Get();
	FLinearColor Color = FillColor.Get();

	SSimpleTimeSlider::FScrubRangeToScreen RangeToScreen(DebugTimeRange, AllottedGeometry.GetLocalSize());
	FVector2f Size = AllottedGeometry.GetLocalSize();

	if (TSharedPtr<FSegmentData> SegmentDataPtr = SegmentData.Get())
	{
		uint32 Index = 0;
		const TOptional<TArray<FLinearColor>>& OverrideColors = SegmentDataPtr->AlternatingSegmentsColors;
		const bool bOverrideColorsProvided = OverrideColors.IsSet() && OverrideColors.GetValue().Num() > 0;
		for (const TRange<double>& SegmentRange : SegmentDataPtr->Segments)
		{
			double LowerBound = SegmentRange.HasLowerBound() ? SegmentRange.GetLowerBoundValue() : 0;
			double UpperBound = SegmentRange.HasUpperBound() ? SegmentRange.GetUpperBoundValue() : Size.X;
			float BoxMin = FMath::Max(0.f, RangeToScreen.InputToLocalX(LowerBound));
			float BoxMax = FMath::Min(Size.X, RangeToScreen.InputToLocalX(UpperBound));

			FPaintGeometry BoxGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(BoxMax - BoxMin, Size.Y - 2), FSlateLayoutTransform(FVector2f(BoxMin, 1.f)));

			const FSlateBrush* Brush = FAppStyle::GetBrush("Sequencer.SectionArea.Background");
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId, BoxGeometry, Brush, ESlateDrawEffect::None, bOverrideColorsProvided ? OverrideColors.GetValue()[Index % OverrideColors->Num()] : Color);
			++Index;
		}
		
		++LayerId;
	}

	return LayerId;
}

#undef LOCTEXT_NAMESPACE
