// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSimpleButton.h"

#include "ToolWidgetsStyle.h"
#include "ToolWidgetsUtilitiesPrivate.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

void SSimpleButton::Construct(const FArguments& InArgs)
{
	const FActionButtonStyle* ActionButtonStyle = &UE::ToolWidgets::FToolWidgetsStyle::Get().GetWidgetStyle<FActionButtonStyle>("SimpleButton");

	// Check for widget level override, then style override, otherwise unset
	const TAttribute<const FSlateBrush*> Icon = InArgs._Icon.IsSet()
		? InArgs._Icon
		: ActionButtonStyle->IconBrush.IsSet()
		? &ActionButtonStyle->IconBrush.GetValue()
		: nullptr;

	// Empty/default args will resolve from the ActionButtonStyle
	const TSharedRef<SWidget> ButtonContent =
		UE::ToolWidgets::Private::ActionButton::MakeButtonContent(
			ActionButtonStyle,
			Icon,
			{},
			InArgs._Text,
			{});

	SButton::Construct(
		SButton::FArguments()
		.ContentPadding(ActionButtonStyle->GetButtonContentPadding())
		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>(InArgs._Text.IsSet() ? "SimpleButtonLabelAndIcon" : "SimpleButton"))
		.IsEnabled(InArgs._IsEnabled)
		.ToolTipText(InArgs._ToolTipText)
		.HAlign(static_cast<EHorizontalAlignment>(ActionButtonStyle->HorizontalContentAlignment))
		.VAlign(VAlign_Center)
		.OnClicked(InArgs._OnClicked)
		[
			ButtonContent
		]);
}
