// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSimpleComboButton.h"

#include "Styling/SlateTypes.h"
#include "ToolWidgetsStyle.h"
#include "ToolWidgetsUtilitiesPrivate.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

void SSimpleComboButton::Construct(const FArguments& InArgs)
{
	const FActionButtonStyle* ActionButtonStyle = &UE::ToolWidgets::FToolWidgetsStyle::Get().GetWidgetStyle<FActionButtonStyle>("SimpleComboButton");

	// Check for widget level override, then style override, otherwise unset
	const TAttribute<const FSlateBrush*> Icon = InArgs._Icon.IsSet()
		? InArgs._Icon
		: ActionButtonStyle->IconBrush.IsSet()
		? &ActionButtonStyle->IconBrush.GetValue()
		: nullptr;

	const bool bHasIcon = Icon.Get() || Icon.IsBound();

	// Empty/default args will resolve from the ActionButtonStyle
	const TSharedRef<SWidget> ButtonContent =
		UE::ToolWidgets::Private::ActionButton::MakeButtonContent(
			ActionButtonStyle,
			Icon,
			{},
			InArgs._Text,
			InArgs._UsesSmallText
				? &FAppStyle::GetWidgetStyle<FTextBlockStyle>("SmallText")
				: &FAppStyle::GetWidgetStyle<FTextBlockStyle>("SmallButtonText"));

	const TAttribute<FMargin> ComboButtonContentPadding = ActionButtonStyle->GetComboButtonContentPadding();

	SComboButton::Construct(SComboButton::FArguments()
		.HasDownArrow(InArgs._HasDownArrow)
		.ContentPadding(ComboButtonContentPadding)
		.ButtonStyle(bHasIcon ? &ActionButtonStyle->GetIconButtonStyle() : &ActionButtonStyle->ButtonStyle)
		.ComboButtonStyle(&ActionButtonStyle->ComboButtonStyle)
		.IsEnabled(InArgs._IsEnabled)
		.ToolTipText(InArgs._ToolTipText)
		.HAlign(static_cast<EHorizontalAlignment>(ActionButtonStyle->HorizontalContentAlignment))
		.VAlign(VAlign_Center)
		.ButtonContent()
		[
			ButtonContent
		]
		.MenuContent()
		[
			InArgs._MenuContent.Widget
		]
		.OnGetMenuContent(InArgs._OnGetMenuContent)
		.OnMenuOpenChanged(InArgs._OnMenuOpenChanged)
		.OnComboBoxOpened(InArgs._OnComboBoxOpened)
	);
}

void SSimpleComboButton::SetMenuContentWidgetToFocus(TWeakPtr<SWidget> InWidget)
{
	SComboButton::SetMenuContentWidgetToFocus(InWidget);
}

void SSimpleComboButton::SetIsMenuOpen(bool bInIsOpen, bool bInIsFocused)
{
	SetIsOpen(bInIsOpen, bInIsFocused);
}
