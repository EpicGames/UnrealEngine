// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSimpleTimeSlider.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/DrawElements.h"
#include "Styling/CoreStyle.h"

#define LOCTEXT_NAMESPACE "SSimpleTimeSlider"

namespace ScrubConstants
{
	/** The minimum amount of pixels between each major ticks on the widget */
	const int32 MinPixelsPerDisplayTick = 5;

	/**The smallest number of units between between major tick marks */
	const float MinDisplayTickSpacing = 0.001f;
}

void SSimpleTimeSlider::Construct( const SSimpleTimeSlider::FArguments& InArgs )
{
    ScrubPosition = InArgs._ScrubPosition;
    ViewRange = InArgs._ViewRange;
	ClampRange = InArgs._ClampRange;
    AllowZoom = InArgs._AllowZoom;
    AllowPan = InArgs._AllowPan;
	CursorSize = InArgs._CursorSize;
	MirrorLabels = InArgs._MirrorLabels;
    OnScrubPositionChanged = InArgs._OnScrubPositionChanged;
	OnBeginScrubberMovement = InArgs._OnBeginScrubberMovement;
	OnEndScrubberMovement = InArgs._OnEndScrubberMovement;
    OnViewRangeChanged = InArgs._OnViewRangeChanged;
	OnFormatTickLabel = InArgs._OnFormatTickLabel;
	MinPixelsPerDisplayTick = InArgs._MinPixelsPerDisplayTick.Get(ScrubConstants::MinPixelsPerDisplayTick);
	ClampRangeHighlightColor = InArgs._ClampRangeHighlightColor;
	ClampRangeHighlightSize = InArgs._ClampRangeHighlightSize;
	DesiredSize = InArgs._DesiredSize;
	
	DistanceDragged = 0.0f;
	bDraggingScrubber = false;
	bPanning = false;

	// set clipping on by default, since the OnPaint function is drawing outside the bounds
	Clipping = EWidgetClipping::ClipToBounds;

	ScrubHandleUp = FAppStyle::Get().GetBrush( TEXT( "Sequencer.Timeline.VanillaScrubHandleUp" ) ); 
	ScrubHandleDown = FAppStyle::Get().GetBrush( TEXT( "Sequencer.Timeline.VanillaScrubHandleDown" ) );
	CursorBackground = FAppStyle::Get().GetBrush("Sequencer.SectionArea.Background");
}

int32 SSimpleTimeSlider::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	int32 NewLayer = OnPaintTimeSlider( MirrorLabels.Get(), AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled );

	return FMath::Max( NewLayer, SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, NewLayer, InWidgetStyle, ShouldBeEnabled( bParentEnabled ) ) );
}

FVector2D SSimpleTimeSlider::ComputeDesiredSize( float ) const
{
	return UE::Slate::FDeprecateVector2DResult(DesiredSize);
}

/**
 * Gets the the next spacing value in the series 
 * to determine a good spacing value
 * E.g, .001,.005,.010,.050,.100,.500,1.000,etc
 */
static float GetNextSpacing( uint32 CurrentStep )
{
	if(CurrentStep & 0x01) 
	{
		// Odd numbers
		return FMath::Pow( 10.f, 0.5f*((float)(CurrentStep-1)) + 1.f );
	}
	else 
	{
		// Even numbers
		return 0.5f * FMath::Pow( 10.f, 0.5f*((float)(CurrentStep)) + 1.f );
	}
}

/**
* Determines the optimal spacing between tick marks in the slider for a given pixel density
* Increments until a minimum amount of slate units specified by MinTick is reached
*
* @param InPixelsPerInput	The density of pixels between each input
* @param MinTick			The minimum slate units per tick allowed
* @param MinTickSpacing	The minimum tick spacing in time units allowed
* @return the optimal spacing in time units
*/
static float DetermineOptimalSpacing(float InPixelsPerInput, uint32 InMinTick, float InMinTickSpacing)
{
	if (InPixelsPerInput <= 0.0f)
		return InMinTickSpacing;

	uint32 CurStep = 0;

	// Start with the smallest spacing
	float Spacing = InMinTickSpacing;
	float MinTick = static_cast<float>(InMinTick);

	while( Spacing * InPixelsPerInput < MinTick )
	{
		Spacing = InMinTickSpacing * GetNextSpacing( CurStep );
		CurStep++;
	}

	return Spacing;
}



void SSimpleTimeSlider::DrawTicks( FSlateWindowElementList& OutDrawElements, const struct FScrubRangeToScreen& RangeToScreen, FDrawTickArgs& InArgs ) const
{
	const float Spacing = DetermineOptimalSpacing( RangeToScreen.PixelsPerInput, MinPixelsPerDisplayTick, ScrubConstants::MinDisplayTickSpacing );

	// Sub divisions
	const int32 Divider = 10;
	// For slightly larger halfway tick mark
	const int32 HalfDivider = Divider / 2;
	// Find out where to start from
	int32 OffsetNum = FMath::FloorToInt32(RangeToScreen.ViewInput.GetLowerBoundValue() / Spacing);
	
	FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 8);

	TArray<FVector2f> LinePoints;
	LinePoints.AddUninitialized(2);

	double Seconds = 0;
	while( (Seconds = static_cast<float>(OffsetNum)*Spacing) < RangeToScreen.ViewInput.GetUpperBoundValue() )
	{
		// X position local to start of the widget area
		float XPos = RangeToScreen.InputToLocalX( Seconds );
		uint32 AbsOffsetNum = FMath::Abs(OffsetNum);

		if ( AbsOffsetNum % Divider == 0 )
		{
			FVector2f Offset( XPos, InArgs.TickOffset );
			FVector2f TickSize( 1.0f, InArgs.MajorTickHeight );

			LinePoints[0] = FVector2f(1.0f,1.0f);
			LinePoints[1] = FVector2f(TickSize);

			// lines should not need anti-aliasing
			const bool bAntiAliasLines = false;

			// Draw each tick mark
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				InArgs.StartLayer,
				InArgs.AllottedGeometry.ToPaintGeometry( TickSize, FSlateLayoutTransform(Offset) ),
				LinePoints,
				InArgs.DrawEffects,
				InArgs.TickColor,
				false
				);

			if( !InArgs.bOnlyDrawMajorTicks )
			{
				FString FrameString = OnFormatTickLabel.IsBound()
					? OnFormatTickLabel.Execute(Seconds)
					: (Spacing == ScrubConstants::MinDisplayTickSpacing ? FString::Printf(TEXT("%.3f"), Seconds) : FString::Printf(TEXT("%.2f"), Seconds));

				// Space the text between the tick mark but slightly above
				const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
				FVector2f TextSize = UE::Slate::CastToVector2f(FontMeasureService->Measure(FrameString, SmallLayoutFont));
				FVector2f TextOffset( XPos-(TextSize.X*0.5f), InArgs.bMirrorLabels ? TextSize.Y :  FMath::Abs( InArgs.AllottedGeometry.GetLocalSize().Y - (InArgs.MajorTickHeight+TextSize.Y) ) );

				FSlateDrawElement::MakeText(
					OutDrawElements,
					InArgs.StartLayer+1, 
					InArgs.AllottedGeometry.ToPaintGeometry( TextSize, FSlateLayoutTransform(TextOffset) ), 
					FrameString, 
					SmallLayoutFont, 
					InArgs.DrawEffects,
					InArgs.TickColor 
				);
			}
		}
		else if( !InArgs.bOnlyDrawMajorTicks )
		{
			// Compute the size of each tick mark.  If we are half way between to visible values display a slightly larger tick mark
			const float MinorTickHeight = AbsOffsetNum % HalfDivider == 0 ? 7.0f : 4.0f;

			FVector2f Offset(XPos, InArgs.bMirrorLabels ? 0.0f : FMath::Abs( InArgs.AllottedGeometry.GetLocalSize().Y - MinorTickHeight ) );
			FVector2f TickSize(1.f, MinorTickHeight);

			LinePoints[0] = FVector2f(1.0f,1.0f);
			LinePoints[1] = FVector2f(TickSize);

			const bool bAntiAlias = false;
			// Draw each sub mark
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				InArgs.StartLayer,
				InArgs.AllottedGeometry.ToPaintGeometry( TickSize, FSlateLayoutTransform(Offset) ),
				LinePoints,
				InArgs.DrawEffects,
				InArgs.TickColor,
				bAntiAlias
			);
		}
		// Advance to next tick mark
		++OffsetNum;
	}
}

int32 SSimpleTimeSlider::OnPaintTimeSlider( bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	const bool bEnabled = bParentEnabled;
	const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	TRange<double> LocalViewRange = ViewRange.Get();
	const double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
	const double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();
	const double LocalSequenceLength = LocalViewRangeMax-LocalViewRangeMin;
	
	FVector2f Scale = FVector2f(1.0f,1.0f);
	if ( LocalSequenceLength > 0.0)
	{
		FScrubRangeToScreen RangeToScreen( LocalViewRange, AllottedGeometry.GetLocalSize() );
		
		// Draw ClampRange 

		float LeftClamp =  RangeToScreen.InputToLocalX(ClampRange.Get().GetLowerBoundValue());
		float RightClamp =  RangeToScreen.InputToLocalX(ClampRange.Get().GetUpperBoundValue());
		float Height = AllottedGeometry.GetLocalSize().Y * ClampRangeHighlightSize.Get();

		FPaintGeometry RangeGeometry;
		if (bMirrorLabels)
		{
			RangeGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(RightClamp-LeftClamp, Height), FSlateLayoutTransform(FVector2f(LeftClamp, 0)));
		}
		else
		{
			RangeGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(RightClamp-LeftClamp, AllottedGeometry.GetLocalSize().Y ), FSlateLayoutTransform(FVector2f(LeftClamp, AllottedGeometry.GetLocalSize().Y - Height)));
		}

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			RangeGeometry,
			CursorBackground,
			DrawEffects,
			ClampRangeHighlightColor.Get()
			);
	
		const float MajorTickHeight = 9.0f;
	
		FDrawTickArgs Args;
		Args.AllottedGeometry = AllottedGeometry;
		Args.bMirrorLabels = bMirrorLabels;
		Args.bOnlyDrawMajorTicks = false;
		Args.TickColor = FLinearColor::White;
		Args.ClippingRect = MyCullingRect;
		Args.DrawEffects = DrawEffects;
		Args.StartLayer = LayerId;
		Args.TickOffset = bMirrorLabels ? 0.0f : FMath::Abs( AllottedGeometry.GetLocalSize().Y - MajorTickHeight );
		Args.MajorTickHeight = MajorTickHeight;

		DrawTicks( OutDrawElements, RangeToScreen, Args );

		const float HandleSize = 13.0f;
		float HalfSize = FMath::TruncToFloat(HandleSize/2.0f);

		// Draw the scrub handle
		const float XPos = RangeToScreen.InputToLocalX( ScrubPosition.Get() );

		// Draw cursor size
		const float CursorHalfSize = CursorSize.Get() * 0.5f;
		const int32 CursorLayer = LayerId + 2;
		const float CursorHalfLength = AllottedGeometry.GetLocalSize().X * CursorHalfSize;
		FPaintGeometry CursorGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(2 * CursorHalfLength, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2f(XPos - CursorHalfLength, 0)));

		FLinearColor CursorColor = InWidgetStyle.GetColorAndOpacityTint();
		CursorColor.A = CursorColor.A*0.08f;
		CursorColor.B *= 0.1f;
		CursorColor.G *= 0.2f;
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			CursorLayer,
			CursorGeometry,
			CursorBackground,
			DrawEffects,
			CursorColor
			);
		


		// Should draw above the text
		const int32 ArrowLayer = LayerId + 3;
		FPaintGeometry MyGeometry =	AllottedGeometry.ToPaintGeometry( FVector2f( HandleSize, AllottedGeometry.GetLocalSize().Y ), FSlateLayoutTransform(FVector2f( XPos-HalfSize, 0 )) );
		FLinearColor ScrubColor = InWidgetStyle.GetColorAndOpacityTint();

		// @todo Sequencer this color should be specified in the style
		ScrubColor.A = ScrubColor.A*0.5f;
		ScrubColor.B *= 0.1f;
		ScrubColor.G *= 0.2f;
		FSlateDrawElement::MakeBox( 
			OutDrawElements,
			ArrowLayer, 
			MyGeometry,
			bMirrorLabels ? ScrubHandleUp : ScrubHandleDown,
			DrawEffects, 
			ScrubColor
			);


		return ArrowLayer;
	}

	return LayerId;
}

FReply SSimpleTimeSlider::OnPreviewMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	bool bHandleLeftMouseButton = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;
	bool bHandleRightMouseButton = MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && AllowPan.Get();
	
	DistanceDragged = 0;

	if ( bHandleLeftMouseButton )
	{
		// Always capture mouse if we left or right click on the widget
		const double NewValue = GetTimeAtCursorPosition(MyGeometry, MouseEvent);
		CommitScrubPosition(NewValue, /*bIsScrubbing=*/false);
		return FReply::Handled().CaptureMouse(AsShared()).PreventThrottling();
	}
	else if ( bHandleRightMouseButton )
	{
		return FReply::Handled().CaptureMouse(AsShared());
	}

	return FReply::Unhandled();
}

FReply SSimpleTimeSlider::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	bool bHandleLeftMouseButton = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && HasMouseCapture();
	bool bHandleRightMouseButton = MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && HasMouseCapture() && AllowPan.Get();
	
	if ( bHandleRightMouseButton )
	{
		if (!bPanning)
		{
			// return unhandled in case our parent wants to use our right mouse button to open a context menu
			return FReply::Unhandled().ReleaseMouseCapture();
		}
		
		bPanning = false;
		return FReply::Handled().ReleaseMouseCapture();
	}
	else if ( bHandleLeftMouseButton )
	{
		if( !bDraggingScrubber )
		{
			const double NewValue = GetTimeAtCursorPosition(MyGeometry, MouseEvent);
			CommitScrubPosition( NewValue, /*bIsScrubbing=*/false );
		}

		bDraggingScrubber = false;
		OnEndScrubberMovement.ExecuteIfBound();
		return FReply::Handled().ReleaseMouseCapture();

	}

	return FReply::Unhandled();
}

double SSimpleTimeSlider::GetTimeAtCursorPosition(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const
{
	FScrubRangeToScreen RangeToScreen(ViewRange.Get(), MyGeometry.GetLocalSize());
	FVector2f CursorPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetLastScreenSpacePosition());
	double NewValue = RangeToScreen.LocalXToInput(CursorPos.X);

	double LocalClampMin = ClampRange.Get().GetLowerBoundValue();
	double LocalClampMax = ClampRange.Get().GetUpperBoundValue();

	if (NewValue < LocalClampMin)
	{
		NewValue = LocalClampMin;
	}

	if (NewValue > LocalClampMax)
	{
		NewValue = LocalClampMax;
	}

	return NewValue;
}

FReply SSimpleTimeSlider::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( HasMouseCapture() )
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton))
		{
			if (!bPanning)
			{
				DistanceDragged += FMath::Abs( MouseEvent.GetCursorDelta().X );
				if ( DistanceDragged > FSlateApplication::Get().GetDragTriggerDistance() )
				{
					FReply::Handled().CaptureMouse(AsShared()).UseHighPrecisionMouseMovement(AsShared());
					SoftwareCursorPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetLastScreenSpacePosition());
					bPanning = true;
				}
			}
			else
			{
				SoftwareCursorPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetLastScreenSpacePosition());

				TRange<double> LocalViewRange = ViewRange.Get();
				double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
				double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();

				FScrubRangeToScreen ScaleInfo( LocalViewRange, MyGeometry.GetLocalSize() );
				FVector2f ScreenDelta = MouseEvent.GetCursorDelta();
				FVector2f InputDelta;
				InputDelta.X = ScreenDelta.X/ScaleInfo.PixelsPerInput;

				const TRange<double> NewViewRange = TRange<double>(LocalViewRangeMin - static_cast<double>(InputDelta.X), LocalViewRangeMax - static_cast<double>(InputDelta.X));
				TRange<double> LocalClampRange = ClampRange.Get();

				OnViewRangeChanged.ExecuteIfBound(NewViewRange);

				if (!ViewRange.IsBound())
				{
					// The  output is not bound to a delegate so we'll manage the value ourselves
					ViewRange.Set(NewViewRange);
				}
			}
		}
		else if (MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton ))
		{
			if ( !bDraggingScrubber )
			{
				DistanceDragged += FMath::Abs( MouseEvent.GetCursorDelta().X );
				if ( DistanceDragged > 0/*FSlateApplication::Get().GetDragTriggerDistance()*/ )
				{
					bDraggingScrubber = true;
					OnBeginScrubberMovement.ExecuteIfBound();
				}
			}
			else
			{
				const double NewValue = GetTimeAtCursorPosition(MyGeometry, MouseEvent);
				CommitScrubPosition(NewValue, /*bIsScrubbing=*/true);
			}
		}
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SSimpleTimeSlider::CommitScrubPosition( double NewValue, bool bIsScrubbing )
{
	// Manage the scrub position ourselves if its not bound to a delegate
	if ( !ScrubPosition.IsBound() )
	{
		ScrubPosition.Set( NewValue );
	}

	if (!ViewRange.IsBound())
	{
		TRange<double> LocalViewRange = ViewRange.Get();
		const double RangeSize = LocalViewRange.Size<double>();
		if (NewValue < LocalViewRange.GetLowerBoundValue())
		{
			SetTimeRange(NewValue, NewValue + RangeSize);
		}
		else if (NewValue > LocalViewRange.GetUpperBoundValue())
		{
			SetTimeRange(NewValue - RangeSize, NewValue);
		}
	}

	OnScrubPositionChanged.ExecuteIfBound( NewValue, bIsScrubbing );
}

void SSimpleTimeSlider::SetTimeRange(double NewViewOutputMin, double NewViewOutputMax)
{
	if (!ViewRange.IsBound())
	{
		ViewRange.Set(TRange<double>(NewViewOutputMin, NewViewOutputMax));
	}
	
	OnViewRangeChanged.ExecuteIfBound(TRange<double>(NewViewOutputMin, NewViewOutputMax));
}

void SSimpleTimeSlider::SetClampRange(double MinValue, double MaxValue)
{
	if (!ClampRange.IsBound())
	{
		ClampRange.Set(TRange<double>(MinValue, MaxValue));
	}
}

FReply SSimpleTimeSlider::OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FReply ReturnValue = FReply::Unhandled();

	if ( AllowZoom.Get() && MouseEvent.GetModifierKeys().IsControlDown())
	{
		const float ZoomDelta = -0.1f * MouseEvent.GetWheelDelta();

		float MouseFractionX = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).X / MyGeometry.GetLocalSize().X;

		TRange<double> LocalViewRange = ViewRange.Get();
		double LocalViewRangeMax = LocalViewRange.GetUpperBoundValue();
		double LocalViewRangeMin = LocalViewRange.GetLowerBoundValue();
		const double OutputViewSize = LocalViewRangeMax - LocalViewRangeMin;
		const double OutputChange = OutputViewSize * ZoomDelta;

		double NewViewOutputMin = LocalViewRangeMin - (OutputChange * MouseFractionX);
		double NewViewOutputMax = LocalViewRangeMax + (OutputChange * (1.0f - MouseFractionX));

		if (NewViewOutputMin < NewViewOutputMax)
		{
			double LocalClampMin = ClampRange.Get().GetLowerBoundValue();
			double LocalClampMax = ClampRange.Get().GetUpperBoundValue();

			OnViewRangeChanged.ExecuteIfBound(TRange<double>(NewViewOutputMin, NewViewOutputMax));
			if( !ViewRange.IsBound() )
			{	
				// The  output is not bound to a delegate so we'll manage the value ourselves
				ViewRange.Set( TRange<double>( NewViewOutputMin, NewViewOutputMax ) );
			}
		}

		ReturnValue = FReply::Handled();
	}

	return ReturnValue;
}

#undef LOCTEXT_NAMESPACE
