// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWarningOrErrorBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"

void SWarningOrErrorBox::Construct(const FArguments& InArgs)
{
	MessageStyle = InArgs._MessageStyle;

	SBorder::Construct(SBorder::FArguments()
		.Padding(InArgs._Padding)
		.ForegroundColor(FAppStyle::Get().GetSlateColor("Colors.White"))
		.BorderImage_Lambda([this]() { return MessageStyle.Get() == EMessageStyle::Warning ? FAppStyle::Get().GetBrush("RoundedWarning") : FAppStyle::Get().GetBrush("RoundedError"); })
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(0.0f, 0.0f, 16.0f, 0.0f))
			[
				SNew(SImage)
				.DesiredSizeOverride(InArgs._IconSize)
				.Image_Lambda([this]() { return MessageStyle.Get() == EMessageStyle::Warning ? FAppStyle::Get().GetBrush("Icons.WarningWithColor") : FAppStyle::Get().GetBrush("Icons.ErrorWithColor"); })
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(InArgs._Message)
				.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.White"))
				.AutoWrapText(InArgs._AutoWrapText)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(InArgs._ContentVAlign)
			.HAlign(InArgs._ContentHAlign)
			[
				InArgs._Content.Widget
			]

		]);
}

