// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWidgetDrawer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/Application/SlateApplication.h"
#include "InputCoreTypes.h"
#include "Misc/ConfigCacheIni.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Types/SlateAttributeMetaData.h"

#define LOCTEXT_NAMESPACE "StatusBar"

namespace StatusBarNotificationConstants
{
	// How long progress notification toasts should appear for
	const float NotificationExpireTime = 5.0f;

	const float NotificationFadeDuration = .15f;

	// Delay before a progress notification becomes visible. This is to avoid the status bar to animate and flicker from short lived notifications. 
	const double NotificationDelay = 3.0;
}

class SDrawerOverlay : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SDrawerOverlay)
	{
		_Clipping = EWidgetClipping::ClipToBounds;
		_ShadowOffset = FVector2f(10.0f, 20.0f);
	}
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ARGUMENT(float, MinDrawerHeight)
		SLATE_ARGUMENT(float, MaxDrawerHeight)
		SLATE_ARGUMENT(float, TargetDrawerHeight)
		SLATE_EVENT(FOnStatusBarDrawerTargetHeightChanged, OnTargetHeightChanged)
		SLATE_EVENT(FSimpleDelegate, OnDismissComplete)
		SLATE_ARGUMENT(FDeprecateSlateVector2D, ShadowOffset)
	SLATE_END_ARGS()

	~SDrawerOverlay()
	{
		FSlateThrottleManager::Get().LeaveResponsiveMode(AnimationThrottle);
	}

	void Construct(const FArguments& InArgs)
	{
		CurrentHeight = 0;

		ShadowOffset = InArgs._ShadowOffset;
		ExpanderSize = 5.0f;

		SplitterStyle = &FAppStyle::Get().GetWidgetStyle<FSplitterStyle>("Splitter");

		MinHeight = InArgs._MinDrawerHeight;

		MaxHeight = InArgs._MaxDrawerHeight;

		TargetHeight = FMath::Clamp(InArgs._TargetDrawerHeight, MinHeight, MaxHeight);

		OnTargetHeightChanged = InArgs._OnTargetHeightChanged;

		BackgroundBrush = FAppStyle::Get().GetBrush("StatusBar.DrawerBackground");
		ShadowBrush = FAppStyle::Get().GetBrush("StatusBar.DrawerShadow");
		BorderBrush = FAppStyle::Get().GetBrush("Docking.Sidebar.Border");

		bIsResizeHandleHovered = false;
		bIsResizing = false;

		OnDismissComplete = InArgs._OnDismissComplete;

		DrawerEasingCurve = FCurveSequence(0.0f, 0.15f, ECurveEaseFunction::QuadOut);

		ChildSlot
		[
			InArgs._Content.Widget
		];
	}

	void UpdateHeightInterp(float InAlpha)
	{
		float NewHeight = FMath::Lerp(0.0f, TargetHeight, InAlpha);

		SetHeight(NewHeight);
	}

	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

	virtual FVector2D ComputeDesiredSize(float) const
	{
		return UE::Slate::FDeprecateVector2DResult(1.0f, TargetHeight + ShadowOffset.Y);
	}

	virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override
	{
		const EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();
		if (ArrangedChildren.Accepts(ChildVisibility))
		{
			ArrangedChildren.AddWidget(
				AllottedGeometry.MakeChild(
					ChildSlot.GetWidget(),
					ShadowOffset,
					FVector2f(AllottedGeometry.GetLocalSize().X - (ShadowOffset.X * 2), TargetHeight)
				)
			);
		}
	}

	virtual FReply OnMouseButtonDown(const FGeometry& AllottedGeometry, const FPointerEvent& MouseEvent) override
	{
		FReply Reply = FReply::Unhandled();
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			const FGeometry RenderTransformedChildGeometry = GetRenderTransformedGeometry(AllottedGeometry);
			const FGeometry ResizeHandleGeometry = GetResizeHandleGeometry(AllottedGeometry);

			if (ResizeHandleGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition()))
			{
				bIsResizing = true;
				InitialResizeGeometry = ResizeHandleGeometry;
				InitialHeightAtResize = CurrentHeight;
				ResizeThrottleHandle = FSlateThrottleManager::Get().EnterResponsiveMode();

				Reply = FReply::Handled().CaptureMouse(SharedThis(this));
			}
		}

		return Reply;

	}

	FReply OnMouseButtonUp(const FGeometry& AllottedGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && bIsResizing == true)
		{
			bIsResizing = false;
			FSlateThrottleManager::Get().LeaveResponsiveMode(ResizeThrottleHandle);

			OnTargetHeightChanged.ExecuteIfBound(TargetHeight);
			return FReply::Handled().ReleaseMouseCapture();
		}
		return FReply::Unhandled();
	}

	FReply OnMouseMove(const FGeometry& AllottedGeometry, const FPointerEvent& MouseEvent) override
	{
		const FGeometry ResizeHandleGeometry = GetResizeHandleGeometry(AllottedGeometry);

		bIsResizeHandleHovered = ResizeHandleGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition());

		if (bIsResizing && this->HasMouseCapture() && !MouseEvent.GetCursorDelta().IsZero())
		{
			const FVector2f LocalMousePos = InitialResizeGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
			const float DeltaHeight = (InitialResizeGeometry.GetLocalPositionAtCoordinates(FVector2f::ZeroVector) - LocalMousePos).Y;

			TargetHeight = FMath::Clamp(InitialHeightAtResize + DeltaHeight, MinHeight, MaxHeight);
			SetHeight(InitialHeightAtResize + DeltaHeight);


			return FReply::Handled();
		}
		else
		{
			return FReply::Unhandled();
		}
	}

	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override
	{
		SCompoundWidget::OnMouseLeave(MouseEvent);

		bIsResizeHandleHovered = false;
	}

	FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override
	{
		return bIsResizing || bIsResizeHandleHovered ? FCursorReply::Cursor(EMouseCursor::ResizeUpDown) : FCursorReply::Unhandled();
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		static FSlateColor ShadowColor = FAppStyle::Get().GetSlateColor("Colors.Foldout");

		const FGeometry RenderTransformedChildGeometry = GetRenderTransformedGeometry(AllottedGeometry);
		const FGeometry ResizeHandleGeometry = GetResizeHandleGeometry(AllottedGeometry);

		// Draw the resize handle
		if (bIsResizing || bIsResizeHandleHovered)
		{
			const FSlateBrush* SplitterBrush = &SplitterStyle->HandleHighlightBrush;
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				ResizeHandleGeometry.ToPaintGeometry(),
				SplitterBrush,
				ESlateDrawEffect::None,
				SplitterBrush->GetTint(InWidgetStyle));
		}

		// Top Shadow
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			RenderTransformedChildGeometry.ToPaintGeometry(),
			ShadowBrush,
			ESlateDrawEffect::None,
			ShadowBrush->GetTint(InWidgetStyle));

		// Background
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			RenderTransformedChildGeometry.ToPaintGeometry( FVector2f(AllottedGeometry.GetLocalSize().X - (ShadowOffset.X * 2), TargetHeight), FSlateLayoutTransform(ShadowOffset) ),
			BackgroundBrush,
			ESlateDrawEffect::None,
			BackgroundBrush->GetTint(InWidgetStyle));

		int32 OutLayerId = SCompoundWidget::OnPaint(Args, RenderTransformedChildGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

		// Bottom shadow
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			OutLayerId,
			AllottedGeometry.ToPaintGeometry(FVector2f(AllottedGeometry.GetLocalSize().X, ShadowOffset.Y), FSlateLayoutTransform(FVector2f(0.0f, AllottedGeometry.GetLocalSize().Y - ShadowOffset.Y))),
			ShadowBrush,
			ESlateDrawEffect::None,
			ShadowBrush->GetTint(InWidgetStyle));


		FSlateDrawElement::MakeBox(
			OutDrawElements,
			OutLayerId+1,
			RenderTransformedChildGeometry.ToPaintGeometry(FVector2f(AllottedGeometry.GetLocalSize().X - (ShadowOffset.X * 2), TargetHeight), FSlateLayoutTransform(ShadowOffset)),
			BorderBrush,
			ESlateDrawEffect::None,
			BorderBrush->GetTint(InWidgetStyle));

		return OutLayerId+1;

	}

	void Open()
	{
		DrawerEasingCurve.Play(AsShared(), false, DrawerEasingCurve.IsPlaying() ? DrawerEasingCurve.GetSequenceTime() : 0.0f, false);

		if (!DrawerOpenCloseTimer.IsValid())
		{
			AnimationThrottle = FSlateThrottleManager::Get().EnterResponsiveMode();
			DrawerOpenCloseTimer = RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SDrawerOverlay::UpdateDrawerAnimation));
		}
	}

	void Dismiss()
	{
		if (DrawerEasingCurve.IsForward())
		{
			DrawerEasingCurve.Reverse();
		}

		if (!DrawerOpenCloseTimer.IsValid())
		{
			AnimationThrottle = FSlateThrottleManager::Get().EnterResponsiveMode();
			DrawerOpenCloseTimer = RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SDrawerOverlay::UpdateDrawerAnimation));
		}
	}
private:
	FGeometry GetRenderTransformedGeometry(const FGeometry& AllottedGeometry) const
	{
		return AllottedGeometry.MakeChild(FSlateRenderTransform(FVector2f(0.0f, TargetHeight - CurrentHeight)));
	}

	FGeometry GetResizeHandleGeometry(const FGeometry& AllottedGeometry) const
	{
		return GetRenderTransformedGeometry(AllottedGeometry).MakeChild(
			FVector2f(AllottedGeometry.GetLocalSize().X-ShadowOffset.X*2.f, ExpanderSize),
			FSlateLayoutTransform(ShadowOffset - FVector2f(0.0f, ExpanderSize))
		);
	}

	void SetHeight(float NewHeight)
	{
		CurrentHeight = FMath::Clamp(NewHeight, MinHeight, TargetHeight);
	}

	EActiveTimerReturnType UpdateDrawerAnimation(double CurrentTime, float DeltaTime)
	{
		UpdateHeightInterp(DrawerEasingCurve.GetLerp());

		if (!DrawerEasingCurve.IsPlaying())
		{
			if (DrawerEasingCurve.IsAtStart())
			{
				OnDismissComplete.ExecuteIfBound();
			}

			FSlateThrottleManager::Get().LeaveResponsiveMode(AnimationThrottle);
			DrawerOpenCloseTimer.Reset();
			return EActiveTimerReturnType::Stop;
		}

		return EActiveTimerReturnType::Continue;
	}


private:
	FGeometry InitialResizeGeometry;
	TSharedPtr<FActiveTimerHandle> DrawerOpenCloseTimer;
	FOnStatusBarDrawerTargetHeightChanged OnTargetHeightChanged;
	FCurveSequence DrawerEasingCurve;
	FSimpleDelegate OnDismissComplete;
	const FSlateBrush* BackgroundBrush;
	const FSlateBrush* ShadowBrush;
	const FSlateBrush* BorderBrush;
	const FSplitterStyle* SplitterStyle;
	FVector2f ShadowOffset;
	FThrottleRequest AnimationThrottle;
	FThrottleRequest ResizeThrottleHandle;
	float ExpanderSize;
	float CurrentHeight;
	float MinHeight;
	float MaxHeight;
	float TargetHeight; 
	float InitialHeightAtResize;
	bool bIsResizing;
	bool bIsResizeHandleHovered;
};

SWidgetDrawer::~SWidgetDrawer()
{
	// Ensure the content browser is removed if we're being destroyed
	CloseDrawerImmediately();
}

void SWidgetDrawer::Construct(const FArguments& InArgs, FName InStatusBarName)
{
	DrawerName = InStatusBarName;
	
	FSlateApplication::Get().OnFocusChanging().AddSP(this, &SWidgetDrawer::OnGlobalFocusChanging);
	FGlobalTabmanager::Get()->OnActiveTabChanged_Subscribe(FOnActiveTabChanged::FDelegate::CreateSP(this, &SWidgetDrawer::OnActiveTabChanged));
	FGlobalTabmanager::Get()->OnTabForegrounded_Subscribe(FOnActiveTabChanged::FDelegate::CreateSP(this, &SWidgetDrawer::OnActiveTabChanged));
	
	ChildSlot
	[
		SAssignNew(DrawerBox, SHorizontalBox)
	];
}

void SWidgetDrawer::OnGlobalFocusChanging(const FFocusEvent& FocusEvent, const FWeakWidgetPath& OldFocusedWidgetPath, const TSharedPtr<SWidget>& OldFocusedWidget, const FWidgetPath& NewFocusedWidgetPath, const TSharedPtr<SWidget>& NewFocusedWidget)
{
	// Sometimes when dismissing focus can change which will trigger this again
	static bool bIsRentrant = false; 

	if(!bIsRentrant)
	{
		TGuardValue<bool> RentrancyGuard(bIsRentrant, true);

		TSharedRef<SWidget> ThisWidget = AsShared();

		TSharedPtr<SWidget> ActiveDrawerOverlayContent;
		if (OpenedDrawer.IsValid())
		{
			ActiveDrawerOverlayContent = OpenedDrawer.DrawerOverlay;
		}

		bool bShouldDismiss = false;

		// If we aren't focusing any new widgets, act as if the drawer is in the path 
		bool bDrawerInPath = NewFocusedWidgetPath.ContainsWidget(ActiveDrawerOverlayContent.Get()) 
			|| NewFocusedWidgetPath.ContainsWidget(this) 
			|| NewFocusedWidgetPath.Widgets.Num() == 0;

		// Do not close due to slow tasks as those opening send window activation events
		if (!GIsSlowTask && !bDrawerInPath && !FSlateApplication::Get().GetActiveModalWindow().IsValid() && ActiveDrawerOverlayContent.IsValid())
		{
			if (TSharedPtr<SWidget> MenuHost = FSlateApplication::Get().GetMenuHostWidget())
			{
				FWidgetPath MenuHostPath;

				// See if the menu being opened is part of the content browser path and if so the menu should not be dismissed
				FSlateApplication::Get().GeneratePathToWidgetUnchecked(MenuHost.ToSharedRef(), MenuHostPath, EVisibility::Visible);
				if (!MenuHostPath.ContainsWidget(ActiveDrawerOverlayContent.Get()))
				{
					bShouldDismiss = true;
				}
			}
			// When the focus change is initiated by the window, don't dismiss the drawer. 
			// Scenario: when users try to open the Output Log Drawer via the "View Output Log" hyperlink, and the Output Log Drawer 
			// wasn't in focus already, the window will dismiss the drawer immediately after it's opened due to focus change. 
			else if (FocusEvent.GetCause() != EFocusCause::WindowActivate)
			{
				bShouldDismiss = true;
			}
		}

		if (bShouldDismiss)
		{
			DismissDrawer(NewFocusedWidget);
		}
	}
}

void SWidgetDrawer::OnActiveTabChanged(TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated)
{
	bool bShouldRemoveDrawer = false;
	if (NewlyActivated)
	{
		if (NewlyActivated->GetTabRole() == ETabRole::MajorTab)
		{
			// Remove the drawer if a newly activated tab is a major tab
			bShouldRemoveDrawer = true;
		}
		else if (PreviouslyActive && PreviouslyActive->GetTabManagerPtr() != NewlyActivated->GetTabManagerPtr())
		{
			// Remove the drawer if we're switching tab managers (indicates a new status bar is becoming active)
			bShouldRemoveDrawer = true;
		}
	}

	if (bShouldRemoveDrawer)
	{
		CloseDrawerImmediately();
	}
}

TSharedRef<SWidget> SWidgetDrawer::MakeStatusBarDrawerButton(const FWidgetDrawerConfig& Drawer)
{
	const FName DrawerId = Drawer.UniqueId;

	const FSlateBrush* StatusBarBackground = FAppStyle::Get().GetBrush("Brushes.Panel");

	TSharedRef<SWidget> DrawerButton = 

		SNew(SBorder)
		.Padding(FMargin(2.0f, 0.0f))
		.BorderImage(StatusBarBackground)
		.Visibility(EVisibility::SelfHitTestInvisible)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.IsFocusable(false)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("StatusBar.StatusBarButton"))
			.OnClicked(this, &SWidgetDrawer::OnDrawerButtonClicked, DrawerId)
			.ToolTipText(Drawer.ToolTipText)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(2.0f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(Drawer.Icon)
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(2.0f)
				[
					SNew(STextBlock)
					.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
					.Text(Drawer.ButtonText)
				]
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					Drawer.CustomButtonWidgets
				]
			]
		];


	if (Drawer.CustomWidget)
	{
		auto IsCustomWidgetBorderVisible = [CustomWidgetWeak = Drawer.CustomWidget.ToWeakPtr()]()
		{
			if (TSharedPtr<SWidget> CustomWidget = CustomWidgetWeak.Pin())
			{
				FSlateAttributeMetaData::UpdateOnlyVisibilityAttributes(*CustomWidget, FSlateAttributeMetaData::EInvalidationPermission::AllowInvalidationIfConstructed);
				if (CustomWidget->GetVisibility() == EVisibility::Collapsed)
				{
					return EVisibility::Collapsed;
				}
			}
			return EVisibility::SelfHitTestInvisible;
		};

		return
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				DrawerButton
			]
			+ SHorizontalBox::Slot()
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SBorder)
				.Padding(FMargin(2.0f, 0.0f))
				.BorderImage(StatusBarBackground)
				.Visibility(MakeAttributeLambda(IsCustomWidgetBorderVisible))
				.VAlign(VAlign_Center)
				[
					Drawer.CustomWidget.ToSharedRef()
				]
			];
	
	}
	else
	{
		return DrawerButton;
	}
}

bool SWidgetDrawer::IsDrawerOpened(const FName DrawerId) const
{
	return OpenedDrawer == DrawerId ? true : false;
}

bool SWidgetDrawer::IsAnyOtherDrawerOpened(const FName DrawerId) const
{
	return OpenedDrawer.IsValid() && OpenedDrawer.DrawerId != DrawerId ? true : false;
}

FString SWidgetDrawer::GetSerializableName() const
{
	return DrawerName.GetPlainNameString();
}

FReply SWidgetDrawer::OnDrawerButtonClicked(const FName DrawerId)
{
	if (!IsDrawerOpened(DrawerId))
	{
		OpenDrawer(DrawerId);
	}
	else
	{
		DismissDrawer(nullptr);
	}

	return FReply::Handled();
}

void SWidgetDrawer::OnDrawerHeightChanged(float TargetHeight)
{
	if (TSharedPtr<SWindow> MyWindow = OpenedDrawer.WindowWithOverlayContent.Pin())
	{
		// Save the height has a percentage of the screen
		const float TargetDrawerHeightPct = TargetHeight / (MyWindow->GetSizeInScreen().Y / MyWindow->GetDPIScaleFactor());
		GConfig->SetFloat(TEXT("DrawerSizes"), *(GetSerializableName() + TEXT(".") + OpenedDrawer.DrawerId.ToString()), TargetDrawerHeightPct, GEditorSettingsIni);
	}
}

void SWidgetDrawer::CloseDrawerImmediatelyInternal(const FOpenDrawerData& Data)
{
	if (Data.IsValid())
	{
		TSharedRef<SWidget> DrawerOverlayContent = Data.DrawerOverlay.ToSharedRef();

		// Remove the content browser from the window
		if (TSharedPtr<SWindow> Window = Data.WindowWithOverlayContent.Pin())
		{
			Window->RemoveOverlaySlot(DrawerOverlayContent);
		}
	}
}

void SWidgetDrawer::RegisterDrawer(FWidgetDrawerConfig&& Drawer, int32 SlotIndex)
{
	const int32 NumDrawers = RegisteredDrawers.Num();
	RegisteredDrawers.AddUnique(Drawer);

	if (RegisteredDrawers.Num() > NumDrawers)
	{
		TSharedRef<SWidget> Content = SNew(SBox)
			.Padding(1.0f, 0.0f)
			.Visibility_Lambda([Delegate = Drawer.GetDrawerButtonVisibilityDelegate, Name = Drawer.UniqueId]()
				{
					return Delegate.IsBound() ? Delegate.Execute(Name) : EVisibility::Visible;
				}
			)
			[
				MakeStatusBarDrawerButton(Drawer)
			];

		DrawerIdToContentWidget.Add(Drawer.UniqueId, Content);

		DrawerBox->InsertSlot(SlotIndex)
		.AutoWidth()
		[
			Content
		];
	}
}

void SWidgetDrawer::UnregisterDrawer(FName DrawerId)
{
	if (IsDrawerOpened(DrawerId))
	{
		CloseDrawerImmediately(DrawerId);
	}

	RegisteredDrawers.Remove(DrawerId);

	TWeakPtr<SWidget> ContentWidgetWeak;
	DrawerIdToContentWidget.RemoveAndCopyValue(DrawerId, ContentWidgetWeak);

	if (TSharedPtr<SWidget> ContentWidget = ContentWidgetWeak.Pin())
	{
		DrawerBox->RemoveSlot(ContentWidget.ToSharedRef());
	}
}

void SWidgetDrawer::OpenDrawer(const FName DrawerId)
{
	// Close any other open drawer
	if (OpenedDrawer.DrawerId != DrawerId && DismissingDrawers.IndexOfByKey(DrawerId) == INDEX_NONE)
	{
		DismissDrawer(nullptr);

		FWidgetDrawerConfig* DrawerData = RegisteredDrawers.FindByKey(DrawerId);

		if(DrawerData)
		{
			TSharedRef<SWidgetDrawer> ThisStatusBar = SharedThis(this);

			TSharedPtr<SWindow> MyWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());

			// OpenDrawer is called on deferred mouse events so the window might already be closed by the time we get here
			if (MyWindow == nullptr)
			{
				return;
			}

			const float MaxDrawerHeight = MyWindow->GetSizeInScreen().Y * 0.90f;

			float TargetDrawerHeightPct = .33f;
			GConfig->GetFloat(TEXT("DrawerSizes"), *(GetSerializableName()+TEXT(".")+DrawerData->UniqueId.ToString()), TargetDrawerHeightPct, GEditorSettingsIni);

			float TargetDrawerHeight = (MyWindow->GetSizeInScreen().Y * TargetDrawerHeightPct) / MyWindow->GetDPIScaleFactor();

			const float MinDrawerHeight = GetTickSpaceGeometry().GetLocalSize().Y + MyWindow->GetWindowBorderSize().Bottom;
	
			FOpenDrawerData NewlyOpenedDrawer;

			MyWindow->AddOverlaySlot()
				.VAlign(VAlign_Bottom)
				.Padding(FMargin(10.0f, 20.0f, 10.0f, MinDrawerHeight))
				[
					SAssignNew(NewlyOpenedDrawer.DrawerOverlay, SDrawerOverlay)
					.MinDrawerHeight(MinDrawerHeight)
					.TargetDrawerHeight(TargetDrawerHeight)
					.MaxDrawerHeight(MaxDrawerHeight)
					.OnDismissComplete_Lambda(
						[DrawerId, this]()
						{
							CloseDrawerImmediately(DrawerId);
						})
					.OnTargetHeightChanged(this, &SWidgetDrawer::OnDrawerHeightChanged)
					[
						DrawerData->GetDrawerContentDelegate.Execute()
					]
				];

			NewlyOpenedDrawer.WindowWithOverlayContent = MyWindow;
			NewlyOpenedDrawer.DrawerId = DrawerId;
			NewlyOpenedDrawer.DrawerOverlay->Open();

			OpenedDrawer = MoveTemp(NewlyOpenedDrawer);

			DrawerData->OnDrawerOpenedDelegate.ExecuteIfBound(ThisStatusBar->DrawerName);
		}
	}
}

bool SWidgetDrawer::DismissDrawer(const TSharedPtr<SWidget>& NewlyFocusedWidget)
{
	bool bWasDismissed = false;
	if (OpenedDrawer.IsValid())
	{
		FWidgetDrawerConfig* Drawer = RegisteredDrawers.FindByKey(OpenedDrawer.DrawerId);

		OpenedDrawer.DrawerOverlay->Dismiss();
		DismissingDrawers.Add(MoveTemp(OpenedDrawer));

		OpenedDrawer = FOpenDrawerData();

		Drawer->OnDrawerDismissedDelegate.ExecuteIfBound(NewlyFocusedWidget);
		bWasDismissed = true;
	}

	return bWasDismissed;
}

void SWidgetDrawer::CloseDrawerImmediately(FName DrawerId)
{
	// If no ID is specified remove all drawers
	if (DrawerId.IsNone())
	{
		for (const FOpenDrawerData& Data : DismissingDrawers)
		{
			CloseDrawerImmediatelyInternal(Data);
		}

		DismissingDrawers.Empty();

		CloseDrawerImmediatelyInternal(OpenedDrawer);

		OpenedDrawer = FOpenDrawerData();
	}
	else
	{
		int32 Index = DismissingDrawers.IndexOfByKey(DrawerId);
		if (Index != INDEX_NONE)
		{
			CloseDrawerImmediatelyInternal(DismissingDrawers[Index]);
			DismissingDrawers.RemoveAtSwap(Index);
		}
		else if (OpenedDrawer == DrawerId)
		{
			CloseDrawerImmediatelyInternal(OpenedDrawer);
			OpenedDrawer = FOpenDrawerData();
		}
	}
}

#undef LOCTEXT_NAMESPACE