// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sidebar/SSidebar.h"
#include "Framework/Application/SlateApplication.h"
#include "SidebarButtonMenuContext.h"
#include "Sidebar/ISidebarDrawerContent.h"
#include "Sidebar/SidebarState.h"
#include "Sidebar/SSidebarButton.h"
#include "Sidebar/SSidebarContainer.h"
#include "Sidebar/SSidebarDrawer.h"
#include "Sidebar/SSidebarDrawerContent.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SOverlay.h"

#define LOCTEXT_NAMESPACE "SSidebar"

SSidebar::~SSidebar()
{
}

void SSidebar::Construct(const FArguments& InArgs, const TSharedRef<SSidebarContainer>& InContainerWidget)
{
	ContainerWidgetWeak = InContainerWidget;

	TabLocation = InArgs._TabLocation;
	InContainerWidget->SidebarSizePercent = InArgs._InitialDrawerSize;
	OnGetContent = InArgs._OnGetContent;
	bHideWhenAllDocked = InArgs._HideWhenAllDocked;
	bAlwaysUseMaxButtonSize = InArgs._AlwaysUseMaxButtonSize;
	bDisablePin = InArgs._DisablePin;
	bDisableDock = InArgs._DisableDock;
	OnStateChanged = InArgs._OnStateChanged;

	check(OnGetContent.IsBound());

	SetVisibility(EVisibility::Visible);

	ChildSlot
	[
		SNew(SBox)
		.Padding(FMargin(
			TabLocation == ESidebarTabLocation::Right ? 2.f : 0.f,
			TabLocation == ESidebarTabLocation::Bottom ? 2.f : 0.f,
			TabLocation == ESidebarTabLocation::Left ? 2.f : 0.f,
			TabLocation == ESidebarTabLocation::Top ? 2.f : 0.f))
		.Visibility(this, &SSidebar::GetSidebarVisibility)
		[
			SNew(SBorder)
			.Padding(0.f)
			.BorderImage(FAppStyle::Get().GetBrush(TEXT("Docking.Sidebar.Background")))
			[
				SAssignNew(TabButtonContainer, SScrollBox)
				.Orientation(IsHorizontal() ? Orient_Horizontal : Orient_Vertical)
				.ScrollBarAlwaysVisible(false)
				.ScrollBarVisibility(EVisibility::Collapsed)
			]
		]
	];
}

bool SSidebar::RegisterDrawer(FSidebarDrawerConfig&& InDrawerConfig)
{
	if (ContainsDrawer(InDrawerConfig.UniqueId))
	{
		return false;
	}

	const TSharedRef<FSidebarDrawer> NewDrawer = MakeShared<FSidebarDrawer>(MoveTemp(InDrawerConfig));
	NewDrawer->State = InDrawerConfig.InitialState;
	NewDrawer->bDisablePin = bDisablePin;
	NewDrawer->bDisableDock = bDisableDock;
	NewDrawer->ContentWidget = NewDrawer->Config.OverrideContentWidget.IsValid()
		? NewDrawer->Config.OverrideContentWidget
		: SNew(SSidebarDrawerContent, NewDrawer);
	Drawers.Add(NewDrawer);

	Drawers.Sort([](const TSharedRef<FSidebarDrawer>& InA, const TSharedRef<FSidebarDrawer>& InB) -> bool
		{
			if (InA->Config.SortOrder == InB->Config.SortOrder)
			{
				return InA->Config.ButtonText.Get(FText())
					.CompareTo(InB->Config.ButtonText.Get(FText())) < 0;
			}
			return InA->Config.SortOrder < InB->Config.SortOrder;
		});

	RecreateDrawerWidgets();

	const FName DrawerId = NewDrawer->GetUniqueId();

	if (NewDrawer->State.bIsPinned)
	{
		UndockAllDrawers();
		SetDrawerPinned(DrawerId, true);
	}
	else if (NewDrawer->State.bIsDocked)
	{
		SetDrawerDocked(DrawerId, true);
	}

	TSharedPtr<SSidebarContainer> ContainerWidget = ContainerWidgetWeak.Pin();
	if (ensure(ContainerWidget.IsValid()))
	{
		ContainerWidget->UpdateDrawerTabAppearance();
	}

	if (bHideWhenAllDocked && !AreAllDrawersDocked())
	{
		SetVisibility(EVisibility::Visible);
	}

	return true;
}

bool SSidebar::UnregisterDrawer(const FName InDrawerId)
{
	TSharedPtr<SSidebarContainer> ContainerWidget = ContainerWidgetWeak.Pin();
	if (ensure(ContainerWidget.IsValid()) && IsDrawerOpened(InDrawerId))
	{
		ContainerWidget->CloseAllDrawerWidgets(false);
	}

	const int32 IndexToRemove = Drawers.IndexOfByPredicate(
		[InDrawerId](const TSharedRef<FSidebarDrawer>& InDrawer)
		{
			return InDrawerId == InDrawer->GetUniqueId();
		});
	if (IndexToRemove == INDEX_NONE)
	{
		return false;
	}

	RemoveDrawer(Drawers[IndexToRemove]);

	if (ContainerWidget.IsValid())
	{
		ContainerWidget->SummonPinnedTabIfNothingOpened();
	}

	// Clear the pinned flag when the tab is removed from the sidebar.
	// (Users probably expect that pinning a tab, restoring it/closing it,
	// then moving it to the sidebar again will leave it unpinned the second time.)
	SetDrawerPinned(InDrawerId, false);

	if (Drawers.Num() == 0)
	{
		SetVisibility(EVisibility::Collapsed);
	}
	else if (ContainerWidget.IsValid())
	{
		ContainerWidget->UpdateDrawerTabAppearance();
	}

	return true;
}

bool SSidebar::ContainsDrawer(const FName InDrawerId) const
{
	return FindDrawer(InDrawerId).IsValid();
}

int32 SSidebar::GetDrawerCount() const
{
	return Drawers.Num();
}

EVisibility SSidebar::GetSidebarVisibility() const
{
	// There may be registered drawers that are hidden due to not having any sections,
	// so we cannot just check if Drawers is empty.
	for (const TSharedRef<FSidebarDrawer>& Drawer : Drawers)
	{
		if (Drawer->ContentSections.Num() > 0)
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

void SSidebar::RecreateDrawerWidgets()
{
	TabButtonContainer->ClearChildren();

	for (const TSharedRef<FSidebarDrawer>& Drawer : Drawers)
	{
		const TWeakPtr<FSidebarDrawer> WeakDrawer = Drawer;

		TabButtonContainer->AddSlot()
			[
				SAssignNew(Drawer->ButtonWidget, SSidebarButton, Drawer, GetTabLocation())
				.Visibility_Lambda([WeakDrawer]()
					{
						return WeakDrawer.IsValid() && WeakDrawer.Pin()->ContentSections.IsEmpty()
							? EVisibility::Collapsed : EVisibility::Visible;
					})
				.MinButtonSize(bAlwaysUseMaxButtonSize ? MaxTabButtonSize : MinTabButtonSize)
				.MaxButtonSize(MaxTabButtonSize)
				.ButtonThickness(TabButtonThickness)
				.OnPressed(this, &SSidebar::OnTabDrawerButtonPressed)
				.OnPinToggled(this, &SSidebar::OnDrawerTabPinToggled)
				.OnDockToggled(this, &SSidebar::OnDrawerTabDockToggled)
				.OnGetContextMenuContent(this, &SSidebar::OnGetTabDrawerContextMenuWidget, Drawer)
			];
	}
}

bool SSidebar::RegisterDrawerSection(const FName InDrawerId, const TSharedPtr<ISidebarDrawerContent>& InSection)
{
	const TSharedPtr<FSidebarDrawer> Drawer = FindDrawer(InDrawerId);
	if (!Drawer.IsValid())
	{
		return false;
	}

	const FName SectionUniqueId = InSection->GetUniqueId();
	if (Drawer->ContentSections.Contains(SectionUniqueId))
	{
		return false;
	}

	Drawer->ContentSections.Add(SectionUniqueId, InSection.ToSharedRef());

	const TSharedPtr<SSidebarDrawerContent> DrawerSection = StaticCastSharedPtr<SSidebarDrawerContent>(Drawer->ContentWidget);
	if (DrawerSection.IsValid())
	{
		DrawerSection->BuildContent();
	}

	return true;
}

bool SSidebar::UnregisterDrawerSection(const FName InDrawerId, const FName InSectionId)
{
	const TSharedPtr<FSidebarDrawer> Drawer = FindDrawer(InDrawerId);
	if (!Drawer.IsValid())
	{
		return false;
	}

	if (!Drawer->ContentSections.Contains(InSectionId))
	{
		return false;
	}

	Drawer->ContentSections.Remove(InSectionId);

	return true;
}

bool SSidebar::TryOpenDrawer(const FName InDrawerId)
{
	if (IsDrawerOpened(InDrawerId) || IsDrawerDocked(InDrawerId))
	{
		return false;
	}

	const TSharedPtr<FSidebarDrawer> Drawer = FindDrawer(InDrawerId);
	if (!Drawer.IsValid())
	{
		return false;
	}

	TSharedPtr<SSidebarContainer> ContainerWidget = ContainerWidgetWeak.Pin();
	if (ensure(ContainerWidget.IsValid()))
	{
		ContainerWidget->OpenDrawerNextFrame(Drawer.ToSharedRef());
	}

	return true;
}

void SSidebar::CloseAllDrawers(const bool bInAnimate)
{
	TSharedPtr<SSidebarContainer> ContainerWidget = ContainerWidgetWeak.Pin();
	if (ensure(ContainerWidget.IsValid()))
	{
		ContainerWidget->CloseAllDrawerWidgets(bInAnimate);
	}
}

void SSidebar::OnTabDrawerButtonPressed(const TSharedRef<FSidebarDrawer>& InDrawer)
{
	TSharedPtr<SSidebarContainer> ContainerWidget = ContainerWidgetWeak.Pin();

	if (InDrawer->bIsOpen)
	{
		// When clicking on the button of an active (but unpinned) tab, close that tab drawer
		if (!IsDrawerPinned(InDrawer->GetUniqueId()))
		{
			if (ensure(ContainerWidget.IsValid()))
			{
				ContainerWidget->CloseDrawer_Internal(InDrawer);
			}
		}
		else if (!InDrawer->DrawerWidget->HasKeyboardFocus())
		{
			FSlateApplication::Get().SetKeyboardFocus(InDrawer->DrawerWidget);
		}
	}
	else if (!InDrawer->State.bIsDocked && ensure(ContainerWidget.IsValid()))
	{
		// Otherwise clicking on an inactive tab should open the drawer
		ContainerWidget->OpenDrawer_Internal(InDrawer);
	}
}

void SSidebar::OnDrawerTabPinToggled(const TSharedRef<FSidebarDrawer>& InDrawer, const bool bIsPinned)
{
	// Set pin state for given tab; clear the pin state for all other tabs
	for (const TSharedRef<FSidebarDrawer>& DrawerTab : Drawers)
	{
		SetDrawerPinned(DrawerTab->GetUniqueId(), DrawerTab == InDrawer ? bIsPinned : false);
	}
}

void SSidebar::OnDrawerTabDockToggled(const TSharedRef<FSidebarDrawer>& InDrawer, const bool bIsDocked)
{
	SetDrawerDocked(InDrawer->GetUniqueId(), bIsDocked);

	if (!bIsDocked)
	{
		SetWidgetDrawerSize(InDrawer);
	}
}

TSharedRef<SWidget> SSidebar::OnGetTabDrawerContextMenuWidget(TSharedRef<FSidebarDrawer> InDrawer)
{
	UToolMenus* const ToolMenus = UToolMenus::Get();
	if (!IsValid(ToolMenus))
	{
		return SNullWidget::NullWidget;
	}

	static constexpr const TCHAR* MenuName = TEXT("SidebarTabMenu");

	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* const NewMenu = ToolMenus->RegisterMenu(MenuName, NAME_None, EMultiBoxType::Menu);
		check(IsValid(NewMenu));
		
		NewMenu->AddDynamicSection(TEXT("Options"), FNewToolMenuDelegate::CreateSP(this, &SSidebar::BuildOptionsMenu));
	}

	USidebarButtonMenuContext* const ContextObject = NewObject<USidebarButtonMenuContext>();
	ContextObject->Init(SharedThis(this), InDrawer);

	const FToolMenuContext MenuContext(nullptr, nullptr, ContextObject);
	return ToolMenus->GenerateWidget(MenuName, MenuContext);
}

void SSidebar::BuildOptionsMenu(UToolMenu* const InMenu)
{
	if (!IsValid(InMenu))
	{
		return;
	}

	USidebarButtonMenuContext* const ContextMenu = InMenu->FindContext<USidebarButtonMenuContext>();
	if (!IsValid(ContextMenu))
	{
		return;
	}

	const TSharedPtr<FSidebarDrawer> Drawer = ContextMenu->GetDrawer();
	if (!Drawer.IsValid())
	{
		return;
	}

	FToolMenuSection& Section = InMenu->FindOrAddSection(TEXT("Options"), LOCTEXT("Options", "Options"));

	if (Drawer->State.bIsDocked)
	{
		Section.AddMenuEntry(TEXT("Undock"),
			LOCTEXT("UndockLabel", "Undock"),
			LOCTEXT("UndockToolTip", "Undocks the drawer"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SSidebar::SetDrawerDocked, Drawer->GetUniqueId(), false)));
	}
	else
	{
		Section.AddMenuEntry(TEXT("Dock"),
			LOCTEXT("DockLabel", "Dock"),
			LOCTEXT("DockToolTip", "Docks the drawer"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SSidebar::SetDrawerDocked, Drawer->GetUniqueId(), true)));
	}

	if (Drawer->State.bIsPinned)
	{
		Section.AddMenuEntry(TEXT("Unpin"),
			LOCTEXT("UnpinLabel", "Unpin"),
			LOCTEXT("UnpinTooltip", "Unpins the drawer from always being displayed"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SSidebar::SetDrawerPinned, Drawer->GetUniqueId(), false)));
	}
	else
	{
		Section.AddMenuEntry(TEXT("Pin"),
			LOCTEXT("PinLabel", "Pin"),
			LOCTEXT("PinTooltip", "Pins the drawer to always be displayed"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SSidebar::SetDrawerPinned, Drawer->GetUniqueId(), true)));
	}
}

void SSidebar::RemoveDrawer(const TSharedRef<FSidebarDrawer>& InDrawer)
{
	const FName DrawerId = InDrawer->GetUniqueId();

	const TSharedPtr<SSidebarContainer> ContainerWidget = ContainerWidgetWeak.Pin();
	if (ensure(ContainerWidget.IsValid()) && InDrawer->DrawerWidget.IsValid())
	{
		ContainerWidget->RemoveDrawerOverlaySlot(InDrawer, false);
	}

	if (InDrawer->ButtonWidget.IsValid())
	{
		TabButtonContainer->RemoveSlot(InDrawer->ButtonWidget.ToSharedRef());
	}

	const int32 IndexToRemove = Drawers.IndexOfByPredicate(
		[DrawerId](const TSharedRef<FSidebarDrawer>& InDrawer)
		{
			return DrawerId == InDrawer->GetUniqueId();
		});
	if (IndexToRemove != INDEX_NONE)
	{
		Drawers.RemoveAt(IndexToRemove);
	}

	if (ContainerWidget.IsValid())
	{
		ContainerWidget->UpdateDrawerTabAppearance();
	}
}

void SSidebar::RemoveAllDrawers()
{
	// Iterate over a copy to avoid modifying the array during iteration
	const TArray<TSharedRef<FSidebarDrawer>> DrawersCopy = Drawers;
	for (const TSharedRef<FSidebarDrawer>& Drawer : DrawersCopy)
	{
		RemoveDrawer(Drawer);
	}

	Drawers.Empty();
}

TSharedPtr<FSidebarDrawer> SSidebar::FindDrawer(const FName InDrawerId) const
{
	const TSharedRef<FSidebarDrawer>* const FoundDrawer = Drawers.FindByPredicate(
		[InDrawerId](const TSharedRef<FSidebarDrawer>& InDrawer)
		{
			return InDrawerId == InDrawer->GetUniqueId();
		});
	return FoundDrawer ? *FoundDrawer : TSharedPtr<FSidebarDrawer>();
}

bool SSidebar::HasDrawerOpened() const
{
	for (const TSharedRef<FSidebarDrawer>& DrawerTab : Drawers)
	{
		if (DrawerTab->bIsOpen)
		{
			return true;
		}
	}
	return false;
}

bool SSidebar::IsDrawerOpened(const FName InDrawerId) const
{
	for (const TSharedRef<FSidebarDrawer>& DrawerTab : Drawers)
	{
		if (DrawerTab->bIsOpen && DrawerTab->GetUniqueId() == InDrawerId)
		{
			return true;
		}
	}
	return false;
}

FName SSidebar::GetOpenedDrawerId() const
{
	TSharedPtr<SSidebarContainer> ContainerWidget = ContainerWidgetWeak.Pin();
	if (ensure(ContainerWidget.IsValid()))
	{
		return ContainerWidget->GetOpenedDrawerId();
	}
	return NAME_None;
}

bool SSidebar::HasDrawerPinned() const
{
	for (const TSharedRef<FSidebarDrawer>& DrawerTab : Drawers)
	{
		if (DrawerTab->State.bIsPinned)
		{
			return true;
		}
	}
	return false;
}

bool SSidebar::IsDrawerPinned(const FName InDrawerId) const
{
	const TSharedPtr<FSidebarDrawer> Drawer = FindDrawer(InDrawerId);
	if (!Drawer.IsValid())
	{
		return false;
	}
	return Drawer->State.bIsPinned;
}

TSet<FName> SSidebar::GetPinnedDrawerIds() const
{
	TSet<FName> OutDrawerIds;

	for (const TSharedRef<FSidebarDrawer>& Drawer : Drawers)
	{
		if (Drawer->State.bIsPinned)
		{
			OutDrawerIds.Add(Drawer->GetUniqueId());
		}
	}

	return OutDrawerIds;
}

void SSidebar::SetDrawerPinned(const FName InDrawerId, const bool bInIsPinned)
{
	const TSharedPtr<FSidebarDrawer> DrawerToPin = FindDrawer(InDrawerId);
	if (!DrawerToPin.IsValid())
	{
		return;
	}

	if (bInIsPinned)
	{
		UnpinAllDrawers();

		if (DrawerToPin->State.bIsDocked)
		{
			SetDrawerDocked(InDrawerId, false);
		}

		if (!DrawerToPin->bIsOpen)
		{
			TSharedPtr<SSidebarContainer> ContainerWidget = ContainerWidgetWeak.Pin();
			if (ensure(ContainerWidget.IsValid()))
			{
				ContainerWidget->OpenDrawerNextFrame(DrawerToPin.ToSharedRef(), false);
			}
		}

		// In case two modules attempt to register drawers with initially pinned states
		for (const TSharedRef<FSidebarDrawer>& Drawer : Drawers)
		{
			Drawer->State.bIsPinned = false;
		}
	}

	DrawerToPin->State.bIsPinned = bInIsPinned;
	if (DrawerToPin->State.bIsPinned)
	{
		DrawerToPin->bIsOpen = true;
	}

	OnStateChanged.ExecuteIfBound(GetState());
}

bool SSidebar::HasDrawerDocked() const
{
	for (const TSharedRef<FSidebarDrawer>& Drawer : Drawers)
	{
		if (Drawer->State.bIsDocked)
		{
			return true;
		}
	}
	return false;
}

bool SSidebar::IsDrawerDocked(const FName InDrawerId) const
{
	const TSharedPtr<FSidebarDrawer> Drawer = FindDrawer(InDrawerId);
	if (!Drawer.IsValid())
	{
		return false;
	}
	return Drawer->State.bIsDocked;
}

TSet<FName> SSidebar::GetDockedDrawerIds() const
{
	TSet<FName> OutDrawerIds;

	for (const TSharedRef<FSidebarDrawer>& Drawer : Drawers)
	{
		if (Drawer->State.bIsDocked)
		{
			OutDrawerIds.Add(Drawer->GetUniqueId());
		}
	}

	return OutDrawerIds;
}

void SSidebar::SetDrawerDocked(const FName InDrawerId, const bool bInIsDocked)
{
	const TSharedPtr<FSidebarDrawer> DrawerToDock = FindDrawer(InDrawerId);
	if (!DrawerToDock.IsValid())
	{
		return;
	}

	const TSharedPtr<SSidebarContainer> ContainerWidget = ContainerWidgetWeak.Pin();
	if (!ensure(ContainerWidget.IsValid()))
	{
		return;
	}

	// Unpin and undock all over tabs
	for (const TSharedRef<FSidebarDrawer>& Drawer : Drawers)
	{
		if (bInIsDocked)
		{
			SetDrawerPinned(Drawer->GetUniqueId(), false);
		}

		ContainerWidget->UndockDrawer_Internal(Drawer);
	}

	if (bInIsDocked)
	{
		if (DrawerToDock->State.bIsPinned)
		{
			SetDrawerPinned(InDrawerId, false);
		}

		ContainerWidget->DockDrawer_Internal(DrawerToDock.ToSharedRef());

		if (DrawerToDock->ContentWidget.IsValid())
		{
			if (bHideWhenAllDocked && AreAllDrawersDocked())
			{
				SetVisibility(EVisibility::Collapsed);
			}
		}
		else
		{
			if (bHideWhenAllDocked && !AreAllDrawersDocked())
			{
				SetVisibility(EVisibility::Visible);
			}
		}
	}
	else
	{
		if (bHideWhenAllDocked && !AreAllDrawersDocked())
        {
        	SetVisibility(EVisibility::Visible);
        }
	}

	OnStateChanged.ExecuteIfBound(GetState());
}

void SSidebar::UndockAllDrawers()
{
	for (const TSharedRef<FSidebarDrawer>& Drawer : Drawers)
	{
		SetDrawerDocked(Drawer->GetUniqueId(), false);
	}
}

void SSidebar::UnpinAllDrawers()
{
	for (const TSharedRef<FSidebarDrawer>& Drawer : Drawers)
	{
		SetDrawerPinned(Drawer->GetUniqueId(), false);
	}
}

bool SSidebar::ContainsDrawerSection(const FName InDrawerId, const FName InDrawerSectionId) const
{
	const TSharedPtr<FSidebarDrawer> Drawer = FindDrawer(InDrawerId);
	if (!Drawer.IsValid())
	{
		return false;
	}

	for (const TPair<FName, TSharedRef<ISidebarDrawerContent>>& DrawerSection : Drawer->ContentSections)
	{
		if (DrawerSection.Value->GetSectionId() == InDrawerSectionId)
		{
			return true;
		}
	}

	return false;
}

bool SSidebar::IsHorizontal() const
{
	return TabLocation == ESidebarTabLocation::Top || TabLocation == ESidebarTabLocation::Bottom;
}

bool SSidebar::IsVertical() const
{
	return TabLocation == ESidebarTabLocation::Left || TabLocation == ESidebarTabLocation::Right;
}

FSidebarState SSidebar::GetState() const
{
	FSidebarState OutState;

	OutState.SetHidden(false);

	TSharedPtr<SSidebarContainer> ContainerWidget = ContainerWidgetWeak.Pin();
	if (ensure(ContainerWidget.IsValid()))
	{
		const float CurrentDrawerSize = ContainerWidget->GetCurrentDrawerSize();
		OutState.SetDrawerSizes(CurrentDrawerSize, 1.f - CurrentDrawerSize);
	}

	for (const TSharedRef<FSidebarDrawer>& Drawer : Drawers)
	{
		OutState.FindOrAddDrawerState(Drawer->State);
	}

	return OutState;
}

ESidebarTabLocation SSidebar::GetTabLocation() const
{
	return TabLocation;
}

TSharedRef<SWidget> SSidebar::GetMainContent() const
{
	return OnGetContent.IsBound() ? OnGetContent.Execute() : SNullWidget::NullWidget;
}

void SSidebar::RebuildDrawer(const FName InDrawerId, const bool bInOnlyIfOpen)
{
	const TSharedPtr<FSidebarDrawer> Drawer = FindDrawer(InDrawerId);
	if (!Drawer.IsValid() || Drawer->Config.OverrideContentWidget.IsValid())
	{
		return;
	}

	if (bInOnlyIfOpen && !IsDrawerOpened(InDrawerId))
	{
		return;
	}

	const TSharedPtr<SSidebarDrawerContent> DrawerContent = StaticCastSharedPtr<SSidebarDrawerContent>(Drawer->ContentWidget);
	if (!DrawerContent.IsValid())
	{
		return;
	}

	DrawerContent->BuildContent();
}

const TArray<TSharedRef<FSidebarDrawer>>& SSidebar::GetAllDrawers() const
{
	return Drawers;
}

void SSidebar::SetWidgetDrawerSize(const TSharedRef<FSidebarDrawer>& InDrawer)
{
	if (!InDrawer->DrawerWidget.IsValid())
	{
		return;
	}

	TSharedPtr<SSidebarContainer> ContainerWidget = ContainerWidgetWeak.Pin();
	if (!ensure(ContainerWidget.IsValid()))
	{
		return;
	}

	const float DrawerOverlayWidth = ContainerWidget->GetOverlaySize().X;
	const float CurrentDrawerSize = ContainerWidget->GetCurrentDrawerSize();
	const float PixelWidth = CurrentDrawerSize * DrawerOverlayWidth;
	InDrawer->DrawerWidget->SetCurrentSize(PixelWidth);
}

bool SSidebar::AreAllDrawersDocked() const
{
	for (const TSharedRef<FSidebarDrawer>& Drawer : Drawers)
	{
		if (!Drawer->State.bIsDocked)
		{
			return false;
		}
	}
	return true;
}

#undef LOCTEXT_NAMESPACE
