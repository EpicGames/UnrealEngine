// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSidebarButton.h"
#include "Framework/Application/SlateApplication.h"
#include "Sidebar/SidebarDrawer.h"
#include "Sidebar/SSidebar.h"
#include "Sidebar/SSidebarButtonText.h"
#include "Sidebar/SSidebarDrawer.h"
#include "Styling/AppStyle.h"
#include "Widgets/Colors/SComplexGradient.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SOverlay.h"

#define LOCTEXT_NAMESPACE "SSidebarDrawerButton"

void SSidebarButton::Construct(const FArguments& InArgs, const TSharedRef<FSidebarDrawer>& InDrawer, const ESidebarTabLocation InTabLocation)
{
	DrawerWeak = InDrawer;
	TabLocation = InTabLocation;

	OnPressed = InArgs._OnPressed;
	OnPinToggled = InArgs._OnPinToggled;
	OnDockToggled = InArgs._OnDockToggled;
	OnGetContextMenuContent = InArgs._OnGetContextMenuContent;

	DockTabStyle = &FAppStyle::Get().GetWidgetStyle<FDockTabStyle>(TEXT("Docking.Tab"));

	static FLinearColor ActiveBorderColor = FAppStyle::Get().GetSlateColor(TEXT("Docking.Tab.ActiveTabIndicatorColor")).GetSpecifiedColor();
	static FLinearColor ActiveBorderColorTransparent = FLinearColor(ActiveBorderColor.R, ActiveBorderColor.G, ActiveBorderColor.B, 0.0f);
	static TArray<FLinearColor> GradientStops{ ActiveBorderColorTransparent, ActiveBorderColor, ActiveBorderColorTransparent };
	
	const bool bIsHorizontal = TabLocation == ESidebarTabLocation::Top || TabLocation == ESidebarTabLocation::Bottom;
	const float MinDesiredWidth = bIsHorizontal ? InArgs._MinButtonSize : InArgs._ButtonThickness;
	const float MinDesiredHeight = bIsHorizontal ? InArgs._ButtonThickness : InArgs._MinButtonSize;
	const float MaxDesiredWidth = bIsHorizontal ? InArgs._MaxButtonSize : InArgs._ButtonThickness;
	const float MaxDesiredHeight = bIsHorizontal ? InArgs._ButtonThickness : InArgs._MaxButtonSize;

	TSharedPtr<SImage> IconWidget;
	if (InDrawer->Config.Icon.IsSet() || InDrawer->Config.Icon.IsBound())
	{
		IconWidget = SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(InDrawer->Config.Icon)
			.DesiredSizeOverride(FVector2D(16, 16));
	}

	Label.Reset();
	if (InDrawer->Config.ButtonText.IsSet() || InDrawer->Config.ButtonText.IsBound())
	{
		SAssignNew(Label, SSidebarButtonText)
			.TextStyle(&DockTabStyle->TabTextStyle)
			.Text(InDrawer->Config.ButtonText)
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			.Clipping(EWidgetClipping::ClipToBounds);
	}

	SAssignNew(PinCheckBox, SCheckBox)
		.Style(FAppStyle::Get(), TEXT("ToggleButtonCheckbox"))
		.Visibility(this, &SSidebarButton::GetPinVisibility)
		.ToolTipText(this, &SSidebarButton::GetPinToolTipText)
		.IsChecked(this, &SSidebarButton::IsPinChecked)
		.OnCheckStateChanged(this, &SSidebarButton::OnPinStateChanged)
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(this, &SSidebarButton::GetPinImage)
		];

	SAssignNew(DockCheckBox, SCheckBox)
		.Style(FAppStyle::Get(), TEXT("ToggleButtonCheckbox"))
		.Visibility(this, &SSidebarButton::GetDockVisibility)
		.ToolTipText(this, &SSidebarButton::GetDockToolTipText)
		.IsChecked(this, &SSidebarButton::IsDockChecked)
		.OnCheckStateChanged(this, &SSidebarButton::OnDockStateChanged)
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(this, &SSidebarButton::GetDockImage)
		];

	TSharedPtr<SWidget> ButtonContent;

	const bool bIsVertical = InTabLocation == ESidebarTabLocation::Left || InTabLocation == ESidebarTabLocation::Right;
	if (bIsVertical)
	{
		const TSharedRef<SVerticalBox> VerticalContent = SNew(SVerticalBox);

		if (IconWidget.IsValid())
		{
			VerticalContent->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(0.f, 2.f, 0.f, 3.f)
				[
					IconWidget.ToSharedRef()
				];
		}

		if (Label.IsValid())
		{
			VerticalContent->AddSlot()
				.FillHeight(1.f)
				.HAlign(HAlign_Center)
				.Padding(0.f, 3.f, 0.f, 3.f)
				[
					Label.ToSharedRef()
				];
		}

		if (!InDrawer->bDisablePin)
		{
			VerticalContent->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.Padding(0.f, 3.f, 0.f, 1.f)
				[
					PinCheckBox.ToSharedRef()
				];
		}

		if (!InDrawer->bDisableDock)
		{
			VerticalContent->AddSlot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.Padding(0.0f, 1.0f, 0.0f, 3.0f)
				[
					DockCheckBox.ToSharedRef()
				];
		}

		ButtonContent = VerticalContent;
	}
	else
	{
		const TSharedRef<SHorizontalBox> HorizontalContent = SNew(SHorizontalBox);

		if (IconWidget.IsValid())
		{
			HorizontalContent->AddSlot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(2.f, 0.f, 3.f, 0.f)
				[
					IconWidget.ToSharedRef()
				];
		}
		
		if (Label.IsValid())
		{
			HorizontalContent->AddSlot()
				.FillWidth(1.f)
				.VAlign(VAlign_Center)
				.Padding(3.f, 0.f, 3.f, 0.f)
				[
					Label.ToSharedRef()
				];
		}
		
		if (!InDrawer->bDisablePin)
		{
			HorizontalContent->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(3.f, 0.f, 1.f, 0.f)
				[
					PinCheckBox.ToSharedRef()
				];
		}

		if (!InDrawer->bDisableDock)
		{
			HorizontalContent->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(1.f, 0.f, 3.f, 0.f)
				[
					DockCheckBox.ToSharedRef()
				];
		}

		ButtonContent = HorizontalContent;
	}
	
	ChildSlot
	.Padding(0)
	[
		SNew(SBox)
		.MinDesiredWidth(MinDesiredWidth)
		.MaxDesiredWidth(MaxDesiredWidth)
		.MinDesiredHeight(MinDesiredHeight)
		.MaxDesiredHeight(MaxDesiredHeight)
		.Clipping(EWidgetClipping::ClipToBounds)
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SAssignNew(MainButton, SButton)
				.ToolTipText(InDrawer->Config.ToolTipText)
				.ContentPadding(FMargin(0.f, DockTabStyle->TabPadding.Top, 0.f, DockTabStyle->TabPadding.Bottom))
				.OnPressed_Lambda([this]()
					{
						// Activate tab on mouse down (not mouse down-up) for consistency with non-sidebar tabs
						OnPressed.ExecuteIfBound(DrawerWeak.Pin().ToSharedRef());
					})
				.ForegroundColor(FSlateColor::UseForeground())
				[
					ButtonContent.ToSharedRef()
				]
			]
			+ SOverlay::Slot()
			[
				SAssignNew(OpenBorder, SBorder)
				.Visibility(EVisibility::HitTestInvisible)
			]
			+ SOverlay::Slot()
			.HAlign(GetHAlignFromTabLocation(TabLocation))
			.VAlign(GetVAlignFromTabLocation(TabLocation))
			[
				SAssignNew(ActiveIndicator, SComplexGradient)
				.DesiredSizeOverride(FVector2D(1.f, 1.f))
				.GradientColors(GradientStops)
				.Orientation(Orient_Horizontal)
				.Visibility(this, &SSidebarButton::GetActiveTabIndicatorVisibility)
			]
		]
	];

	UpdateAppearance(nullptr);
}

void SSidebarButton::UpdateAppearance(const TSharedPtr<FSidebarDrawer>& InLastDrawerOpen)
{
	const TSharedPtr<FSidebarDrawer> ThisDrawer = DrawerWeak.Pin();
	if (!ThisDrawer.IsValid())
	{
		return;
	}

	float LabelRotation;
	FName FocusBorderBrushName;

	switch (TabLocation)
	{
	case ESidebarTabLocation::Left:
		LabelRotation = -90.f;
		FocusBorderBrushName = TEXT("Docking.Sidebar.Border_SquareRight");
		break;
	case ESidebarTabLocation::Right:
		LabelRotation = 90.f;
		FocusBorderBrushName = TEXT("Docking.Sidebar.Border_SquareLeft");
		break;
	default:
		LabelRotation = 0.f;
		FocusBorderBrushName = TEXT("None");
		break;
	}

	if (Label.IsValid())
	{
		Label->SetRotation(LabelRotation);
	}

	// Border when not docked and open
	if (InLastDrawerOpen == ThisDrawer && (!ThisDrawer->State.bIsDocked && ThisDrawer->bIsOpen))
	{
		OpenBorder->SetVisibility(EVisibility::HitTestInvisible);
		OpenBorder->SetBorderImage(FAppStyle::Get().GetBrush(FocusBorderBrushName));
	}
	else
	{
		OpenBorder->SetVisibility(EVisibility::Collapsed);
	}

	// Button style
	if (InLastDrawerOpen == ThisDrawer || ThisDrawer->State.bIsDocked)
	{
		// this button is the one with the tab that is actually opened so show the tab border
		MainButton->SetButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>(TEXT("Docking.SidebarButton.Opened")));
	}
	else
	{
		MainButton->SetButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>(TEXT("Docking.SidebarButton.Closed")));
	}
}

void SSidebarButton::OnTabRenamed(const TWeakPtr<FSidebarDrawer>& InDrawer)
{
	if (ensure(InDrawer == DrawerWeak) && InDrawer.IsValid())
	{
		if (const TSharedPtr<FSidebarDrawer> DrawerConfig = InDrawer.Pin())
		{
			if (Label.IsValid())
			{
				Label->SetText(DrawerConfig->Config.ButtonText);
			}

			MainButton->SetToolTipText(DrawerConfig->Config.ToolTipText);
		}
	}
}

FReply SSidebarButton::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton && OnGetContextMenuContent.IsBound())
	{
		FWidgetPath WidgetPath = InMouseEvent.GetEventPath() != nullptr ? *InMouseEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, OnGetContextMenuContent.Execute(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect::ContextMenu);
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FSlateColor SSidebarButton::GetForegroundColor() const
{
	if (ActiveIndicator->GetVisibility() != EVisibility::Collapsed)
	{
		return DockTabStyle->ActiveForegroundColor;
	}
	else if (IsHovered())
	{
		return DockTabStyle->HoveredForegroundColor;
	}
	return FSlateColor::UseStyle();
}

EVisibility SSidebarButton::GetActiveTabIndicatorVisibility() const
{
	const TSharedPtr<FSidebarDrawer> Drawer = DrawerWeak.Pin();
	if (Drawer.IsValid()
		&& Drawer->bIsOpen
		&& Drawer->DrawerWidget.IsValid()
		&& Drawer->DrawerWidget->HasAnyUserFocusOrFocusedDescendants())
	{
		return EVisibility::HitTestInvisible;
	}
	return EVisibility::Collapsed;
}

EVisibility SSidebarButton::GetPinVisibility() const
{
	const TSharedPtr<FSidebarDrawer> Drawer = DrawerWeak.Pin();
	if (!Drawer.IsValid() || Drawer->bDisablePin)
	{
		return EVisibility::Collapsed;
	}
	return (Drawer->State.bIsPinned || IsHovered() || Drawer->bIsOpen)
		? EVisibility::Visible : EVisibility::Hidden;
}

FText SSidebarButton::GetPinToolTipText() const
{
	const TSharedPtr<FSidebarDrawer> Drawer = DrawerWeak.Pin();
	if (Drawer.IsValid() && Drawer->State.bIsPinned)
	{
		return LOCTEXT("UnpinTabToolTip", "Unpin Tab");
	}
	return LOCTEXT("PinTabToolTip", "Pin Tab");
}

ECheckBoxState SSidebarButton::IsPinChecked() const
{
	const TSharedPtr<FSidebarDrawer> Drawer = DrawerWeak.Pin();
	if (Drawer.IsValid() && Drawer->State.bIsPinned)
	{
		return ECheckBoxState::Checked;
	}
	return ECheckBoxState::Unchecked;
}

const FSlateBrush* SSidebarButton::GetPinImage() const
{
	const TSharedPtr<FSidebarDrawer> Drawer = DrawerWeak.Pin();
	if (Drawer.IsValid() && Drawer->State.bIsPinned)
	{
		return FAppStyle::Get().GetBrush(TEXT("Icons.Pinned"));
	}
	return FAppStyle::Get().GetBrush(TEXT("Icons.Unpinned"));
}

void SSidebarButton::OnPinStateChanged(const ECheckBoxState InNewState)
{
	OnPinToggled.ExecuteIfBound(DrawerWeak.Pin().ToSharedRef(), InNewState == ECheckBoxState::Checked);
}

EVisibility SSidebarButton::GetDockVisibility() const
{
	const TSharedPtr<FSidebarDrawer> Drawer = DrawerWeak.Pin();
	if (!Drawer.IsValid() || Drawer->bDisableDock)
	{
		return EVisibility::Collapsed;
	}
	return (Drawer->State.bIsDocked || IsHovered() || Drawer->bIsOpen) ? EVisibility::Visible : EVisibility::Hidden;
}

FText SSidebarButton::GetDockToolTipText() const
{
	const TSharedPtr<FSidebarDrawer> Drawer = DrawerWeak.Pin();
	if (Drawer.IsValid() && Drawer->State.bIsDocked)
	{
		return LOCTEXT("UndockTabToolTip", "Undock Tab");
	}
	return LOCTEXT("DockTabToolTip", "Dock Tab");
}

ECheckBoxState SSidebarButton::IsDockChecked() const
{
	const TSharedPtr<FSidebarDrawer> Drawer = DrawerWeak.Pin();
	if (Drawer.IsValid() && Drawer->State.bIsDocked)
	{
		return ECheckBoxState::Checked;
	}
	return ECheckBoxState::Unchecked;
}

const FSlateBrush* SSidebarButton::GetDockImage() const
{
	const TSharedPtr<FSidebarDrawer> Drawer = DrawerWeak.Pin();
	if (Drawer.IsValid() && Drawer->State.bIsPinned)
	{
		return FAppStyle::Get().GetBrush(TEXT("Icons.Layout"));
	}
	return FAppStyle::Get().GetBrush(TEXT("Icons.Layout"));
}

void SSidebarButton::OnDockStateChanged(const ECheckBoxState InNewState)
{
	OnDockToggled.ExecuteIfBound(DrawerWeak.Pin().ToSharedRef(), InNewState == ECheckBoxState::Checked);
}

EHorizontalAlignment SSidebarButton::GetHAlignFromTabLocation(const ESidebarTabLocation InTabLocation)
{
	switch (InTabLocation)
	{
	case ESidebarTabLocation::Left:
		return HAlign_Left;
	case ESidebarTabLocation::Right:
		return HAlign_Right;
	case ESidebarTabLocation::Top:
	case ESidebarTabLocation::Bottom:
		return HAlign_Fill;
	}
	return HAlign_Fill;
}

EVerticalAlignment SSidebarButton::GetVAlignFromTabLocation(const ESidebarTabLocation InTabLocation)
{
	switch (InTabLocation)
	{
	case ESidebarTabLocation::Left:
	case ESidebarTabLocation::Right:
		return VAlign_Fill;
	case ESidebarTabLocation::Top:
		return VAlign_Top;
	case ESidebarTabLocation::Bottom:
		return VAlign_Bottom;
	}
	return VAlign_Fill;
}

#undef LOCTEXT_NAMESPACE
