// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Misc/Optional.h"
#include "Sidebar/SidebarDrawer.h"
#include "Sidebar/SSidebar.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SLeafWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FSlateTextBlockLayout;
class SBorder;
class SButton;
class SCheckBox;
class SSidebarButtonText;
class SWidget;
enum class ESidebarTabLocation : uint8;
struct FWidgetDrawerConfig;

DECLARE_DELEGATE_OneParam(FOnSidebarButtonPressed, const TSharedRef<FSidebarDrawer>&);
DECLARE_DELEGATE_TwoParams(FOnSidebarPinToggled, const TSharedRef<FSidebarDrawer>&, const bool /*bInIsPinned*/);
DECLARE_DELEGATE_TwoParams(FOnSidebarDockToggled, const TSharedRef<FSidebarDrawer>&, const bool /*bInIsDocked*/);

class SSidebarButton : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSidebarButton)
	{}
		SLATE_ARGUMENT(float, MinButtonSize)
		SLATE_ARGUMENT(float, MaxButtonSize)
		SLATE_ARGUMENT(float, ButtonThickness)
		SLATE_EVENT(FOnSidebarButtonPressed, OnPressed)
		SLATE_EVENT(FOnSidebarPinToggled, OnPinToggled)
		SLATE_EVENT(FOnSidebarPinToggled, OnDockToggled)
		SLATE_EVENT(FOnGetContent, OnGetContextMenuContent)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FSidebarDrawer>& InDrawer, const ESidebarTabLocation InTabLocation);

	void UpdateAppearance(const TSharedPtr<FSidebarDrawer>& InLastDrawerOpen);

	void OnTabRenamed(const TWeakPtr<FSidebarDrawer>& InDrawer);

	static EHorizontalAlignment GetHAlignFromTabLocation(const ESidebarTabLocation InTabLocation);
	static EVerticalAlignment GetVAlignFromTabLocation(const ESidebarTabLocation InTabLocation);

protected:
	EVisibility GetActiveTabIndicatorVisibility() const;

	EVisibility GetPinVisibility() const;
	FText GetPinToolTipText() const;
	ECheckBoxState IsPinChecked() const;
	const FSlateBrush* GetPinImage() const;
	void OnPinStateChanged(const ECheckBoxState InNewState);

	EVisibility GetDockVisibility() const;
	FText GetDockToolTipText() const;
	ECheckBoxState IsDockChecked() const;
	const FSlateBrush* GetDockImage() const;
	void OnDockStateChanged(const ECheckBoxState InNewState);

	//~ Begin SWidget
	virtual FSlateColor GetForegroundColor() const override;
	virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	//~ End SWidget

	TWeakPtr<FSidebarDrawer> DrawerWeak;
	ESidebarTabLocation TabLocation = ESidebarTabLocation::Right;

	FOnSidebarButtonPressed OnPressed;
	FOnSidebarPinToggled OnPinToggled;
	FOnSidebarDockToggled OnDockToggled;
	FOnGetContent OnGetContextMenuContent;

	const FDockTabStyle* DockTabStyle = nullptr;

	TSharedPtr<SSidebarButtonText> Label;
	TSharedPtr<SWidget> ActiveIndicator;
	TSharedPtr<SBorder> OpenBorder;
	TSharedPtr<SButton> MainButton;

	TSharedPtr<SCheckBox> PinCheckBox;
	TSharedPtr<SCheckBox> DockCheckBox;
};
