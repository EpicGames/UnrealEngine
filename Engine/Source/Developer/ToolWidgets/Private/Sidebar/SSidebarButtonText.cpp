// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSidebarButtonText.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/PlainTextLayoutMarshaller.h"
#include "Widgets/Text/SlateTextBlockLayout.h"

#define LOCTEXT_NAMESPACE "SSidebarButtonText"

void SSidebarButtonText::Construct(const FArguments& InArgs)
{
	Text = InArgs._Text;
	TextStyle = *InArgs._TextStyle;
	AngleDegrees = InArgs._AngleDegrees;

	TextLayoutCache = MakeUnique<FSlateTextBlockLayout>(this
		, FTextBlockStyle::GetDefault()
		, TOptional<ETextShapingMethod>()
		, TOptional<ETextFlowDirection>()
		, FCreateSlateTextLayout()
		, FPlainTextLayoutMarshaller::Create()
		, nullptr);
	
	TextLayoutCache->SetTextOverflowPolicy(InArgs._OverflowPolicy.IsSet() ? InArgs._OverflowPolicy : TextStyle.OverflowPolicy);
}

int32 SSidebarButtonText::OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InCullingRect
	, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const
{
	// We're going to figure out the bounds of the corresponding horizontal text, and then rotate it into a vertical orientation.
	const FVector2D LocalSize = InAllottedGeometry.GetLocalSize();
	const FVector2D DesiredHorizontalTextSize = TextLayoutCache->GetDesiredSize();
	const FVector2D ActualHorizontalTextSize(FMath::Min(DesiredHorizontalTextSize.X, LocalSize.Y), FMath::Min(DesiredHorizontalTextSize.Y, LocalSize.X));

	// Now determine the center of the vertical text by rotating the dimensions of the horizontal text.
	// The center should align it to the top of the widget.
	const FVector2D VerticalTextSize(ActualHorizontalTextSize.Y, ActualHorizontalTextSize.X);
	const FVector2D VerticalTextCenter = VerticalTextSize * 0.5f;

	// Now determine where the horizontal text should be positioned so that it is centered on the vertical text:
	//      +-+
	//      |v|
	//      |e|
	// [ horizontal ]
	//      |r|
	//      |t|
	//      +-+
	const FVector2D HorizontalTextPosition = VerticalTextCenter - (ActualHorizontalTextSize * 0.5f);

	// Define the text's geometry using the horizontal bounds, then rotate it 90/-90 degrees into place to become vertical.
	const FSlateRenderTransform RotationTransform(FSlateRenderTransform(FQuat2D(FMath::DegreesToRadians(AngleDegrees.Get(0.f)))));
	const FGeometry TextGeometry = InAllottedGeometry.MakeChild(ActualHorizontalTextSize
		, FSlateLayoutTransform(HorizontalTextPosition), RotationTransform, FVector2D(0.5f, 0.5f));

	return TextLayoutCache->OnPaint(InArgs, TextGeometry, InCullingRect, OutDrawElements, InLayerId, InWidgetStyle, ShouldBeEnabled(bInParentEnabled));
}

FVector2D SSidebarButtonText::ComputeDesiredSize(const float LayoutScaleMultiplier) const
{
	// The text's desired size reflects the horizontal/untransformed text.
	// Switch the dimensions for vertical text.
	const FVector2D DesiredHorizontalTextSize = TextLayoutCache->ComputeDesiredSize(
		FSlateTextBlockLayout::FWidgetDesiredSizeArgs(
			Text.Get(),
			FText(),
			0.f,
			false,
			ETextWrappingPolicy::DefaultWrapping,
			ETextTransformPolicy::None,
			FMargin(),
			1.f,
			true,
			ETextJustify::Left),
		LayoutScaleMultiplier, TextStyle);
	return FVector2D(DesiredHorizontalTextSize.Y, DesiredHorizontalTextSize.X);
}

void SSidebarButtonText::SetText(const TAttribute<FText>& InText)
{
	Text = InText;
}

void SSidebarButtonText::SetRotation(const TAttribute<float>& InAngleDegrees)
{
	AngleDegrees = InAngleDegrees;
}

#undef LOCTEXT_NAMESPACE
