// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Templates/UniquePtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"
#include "Widgets/Text/SlateTextBlockLayout.h"

class FText;

/**
 * Vertical text block for use in the tab drawer button.
 * Text is aligned to the top of the widget if it fits without clipping;
 * otherwise it is ellipsized and fills the widget height.
 */
class SSidebarButtonText : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SSidebarButtonText)
		: _Text()
		, _TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>(TEXT("NormalText")))
		, _AngleDegrees(0)
		, _OverflowPolicy()
	{}
		SLATE_ATTRIBUTE(FText, Text)
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextStyle)
		SLATE_ATTRIBUTE(float, AngleDegrees)
		SLATE_ARGUMENT(TOptional<ETextOverflowPolicy>, OverflowPolicy)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void SetText(const TAttribute<FText>& InText);

	void SetRotation(const TAttribute<float>& InAngleDegrees);

protected:
	//~ Begin SWidget
	virtual int32 OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InCullingRect
		, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const override;
	virtual FVector2D ComputeDesiredSize(const float InLayoutScaleMultiplier) const override;
	//~ End SWidget

	TAttribute<FText> Text;
	FTextBlockStyle TextStyle;
	TAttribute<float> AngleDegrees;
	
	TUniquePtr<FSlateTextBlockLayout> TextLayoutCache;
};
