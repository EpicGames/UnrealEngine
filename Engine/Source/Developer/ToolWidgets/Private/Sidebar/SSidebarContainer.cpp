// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sidebar/SSidebarContainer.h"
#include "Framework/Application/SlateApplication.h"
#include "Sidebar/ISidebarDrawerContent.h"
#include "Sidebar/SSidebar.h"
#include "Sidebar/SSidebarDrawer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"

#define LOCTEXT_NAMESPACE "SSidebarContainer"

void SSidebarContainer::Construct(const FArguments& InArgs)
{
}

void SSidebarContainer::RebuildSidebar(const TSharedRef<SSidebar>& InSidebarWidget, const FSidebarState& InState)
{
	SidebarWidget = InSidebarWidget;

	Reconstruct(InState);
}

void SSidebarContainer::Reconstruct(const FSidebarState& InState)
{
	TSharedPtr<SWidget> OutWidget;

	if (InState.IsHidden())
	{
		DrawersOverlay.Reset();

		OutWidget = SidebarWidget->GetMainContent();
	}
	else if (InState.IsVisible())
	{
		OutWidget = SNew(SOverlay)
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				ConstructBoxPanel(InState)
			]
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SAssignNew(DrawersOverlay, SOverlay)
			];
	}

	ChildSlot
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			OutWidget.ToSharedRef()
		];
}

TSharedRef<SWidget> SSidebarContainer::ConstructBoxPanel(const FSidebarState& InState)
{
	ConstructSplitterPanel(InState);

	// MainSplitter will be valid here if we have a docked drawer
	const TSharedRef<SWidget> Content = MainSplitter.IsValid()
		? MainSplitter.ToSharedRef()
		: SidebarWidget->GetMainContent();

	if (SidebarWidget->IsVertical())
	{
		const TSharedRef<SHorizontalBox> Box = SNew(SHorizontalBox);

		auto BoxContentSlot = [this, &Content, &Box]()
		{
			Box->AddSlot()
				.FillWidth(1.f)
				[
					Content
				];
		};
		auto BoxSidebarSlot = [this, &Box]()
		{
			Box->AddSlot()
				.AutoWidth()
				[
					SidebarWidget.ToSharedRef()
				];
		};

		const ESidebarTabLocation TabLocation = SidebarWidget->GetTabLocation();
		if (TabLocation == ESidebarTabLocation::Left)
		{
			BoxSidebarSlot();
			BoxContentSlot();
		}
		else if (TabLocation == ESidebarTabLocation::Right)
		{
			BoxContentSlot();
			BoxSidebarSlot();
		}

		return Box;
	}

	if (SidebarWidget->IsHorizontal())
	{
		const TSharedRef<SVerticalBox> Box = SNew(SVerticalBox);

		auto BoxContentSlot = [this, &Content, &Box]()
		{
			Box->AddSlot()
				.FillHeight(1.f)
				[
					Content
				];
		};
		auto BoxSidebarSlot = [this, &Box]()
		{
			Box->AddSlot()
				.AutoHeight()
				[
					SidebarWidget.ToSharedRef()
				];
		};

		const ESidebarTabLocation TabLocation = SidebarWidget->GetTabLocation();
		if (TabLocation == ESidebarTabLocation::Top)
		{
			BoxSidebarSlot();
			BoxContentSlot();
		}
		else if (TabLocation == ESidebarTabLocation::Bottom)
		{
			BoxContentSlot();
			BoxSidebarSlot();
		}

		return Box;
	}

	return SNullWidget::NullWidget;
}

void SSidebarContainer::ConstructSplitterPanel(const FSidebarState& InState)
{
	if (InState.IsVisible() && SidebarWidget->HasDrawerDocked())
	{
		const TSet<FName> DockedDrawerIds = SidebarWidget->GetDockedDrawerIds();
		const FName FirstFoundDrawerId = DockedDrawerIds.IsEmpty() ? NAME_None : DockedDrawerIds.Array()[0];

		MainSplitter = SNew(SSplitter)
			.Orientation(GetSplitterOrientation())
			.OnSplitterFinishedResizing(this, &SSidebarContainer::OnSplitterResized);

		const ESidebarTabLocation TabLocation = SidebarWidget->GetTabLocation();
		if (TabLocation == ESidebarTabLocation::Left || TabLocation == ESidebarTabLocation::Top)
		{
			AddSidebarDockSlot(FirstFoundDrawerId);
			AddContentDockSlot();
		}
		else if (TabLocation == ESidebarTabLocation::Right || TabLocation == ESidebarTabLocation::Bottom)
		{
			AddContentDockSlot();
			AddSidebarDockSlot(FirstFoundDrawerId);
		}
	}
	else
	{
		MainSplitter.Reset();
	}
}

void SSidebarContainer::AddContentDockSlot()
{
	const bool bDrawerDocked = SidebarWidget->HasDrawerDocked();

	if (bDrawerDocked)
	{
		ContentSlotSize = TAttribute<float>::Create([this]()
			{
				return ContentSizePercent;
			});
	}
	else
	{
		ContentSlotSize = {};
	}

	MainSplitter->AddSlot()
		.SizeRule(bDrawerDocked ? SSplitter::FractionOfParent : SSplitter::SizeToContent)
		.Value(ContentSlotSize)
		.OnSlotResized(this, &SSidebarContainer::OnContentSlotResizing)
		[
			SidebarWidget->GetMainContent()
		];
}

void SSidebarContainer::RemoveContentDockSlot()
{
	const int32 SlotIndex = GetContentSlotIndex();
	MainSplitter->RemoveAt(SlotIndex);
}

TSharedRef<SWidget> SSidebarContainer::GetSidebarDrawerContent(const TSharedRef<FSidebarDrawer>& InDrawer) const
{
	if (InDrawer->Config.OverrideContentWidget.IsValid())
	{
		return InDrawer->Config.OverrideContentWidget.ToSharedRef();
	}
	return InDrawer->ContentWidget.IsValid() ? InDrawer->ContentWidget.ToSharedRef() : SNullWidget::NullWidget;
}

void SSidebarContainer::AddSidebarDockSlot(const FName InDockDrawerId)
{
	const TSharedPtr<FSidebarDrawer> DrawerToDock = SidebarWidget->FindDrawer(InDockDrawerId);
	if (!DrawerToDock.IsValid())
	{
		return;
	}

	const bool bDrawerDocked = SidebarWidget->HasDrawerDocked();

	if (bDrawerDocked)
	{
		SidebarSlotSize = TAttribute<float>::Create([this]()
			{
				return SidebarSizePercent;
			});
	}
	else
	{
		SidebarSlotSize = {};
	}

	MainSplitter->AddSlot()
		.SizeRule(bDrawerDocked ? SSplitter::FractionOfParent : SSplitter::SizeToContent)
		.Value(SidebarSlotSize)
		.OnSlotResized(this, &SSidebarContainer::OnSidebarSlotResizing)
		[
			GetSidebarDrawerContent(DrawerToDock.ToSharedRef())
		];
}

void SSidebarContainer::RemoveSidebarDockSlot()
{
	const int32 SlotIndex = GetSidebarSlotIndex();
	MainSplitter->RemoveAt(SlotIndex);
}

float SSidebarContainer::GetContentSlotSize() const
{
	return ContentSizePercent;
}

float SSidebarContainer::GetSidebarSlotSize() const
{
	return SidebarSizePercent;
}

int32 SSidebarContainer::GetContentSlotIndex() const
{
	switch (SidebarWidget->GetTabLocation())
	{
	case ESidebarTabLocation::Right:
	case ESidebarTabLocation::Bottom:
		return 0;
	case ESidebarTabLocation::Left:
	case ESidebarTabLocation::Top:
		return 1;
	}
	return 0;
}

int32 SSidebarContainer::GetSidebarSlotIndex() const
{
	switch (SidebarWidget->GetTabLocation())
	{
	case ESidebarTabLocation::Left:
	case ESidebarTabLocation::Top:
		return 0;
	case ESidebarTabLocation::Right:
	case ESidebarTabLocation::Bottom:
		return 1;
	}
	return 1;
}

EOrientation SSidebarContainer::GetSplitterOrientation() const
{
	switch (SidebarWidget->GetTabLocation())
	{
	case ESidebarTabLocation::Left:
	case ESidebarTabLocation::Right:
		return Orient_Horizontal;
	case ESidebarTabLocation::Top:
	case ESidebarTabLocation::Bottom:
		return Orient_Vertical;
	}
	return Orient_Horizontal;
}

ESidebarTabLocation SSidebarContainer::GetTabLocation() const
{
	return SidebarWidget->GetTabLocation();
}

float SSidebarContainer::GetCurrentDrawerSize() const
{
	return SidebarSizePercent;
}

UE::Slate::FDeprecateVector2DResult SSidebarContainer::GetOverlaySize() const
{
	return DrawersOverlay->GetTickSpaceGeometry().GetLocalSize();
}

bool SSidebarContainer::AddDrawerOverlaySlot(const TSharedRef<FSidebarDrawer>& InDrawer)
{
	if (!InDrawer->DrawerWidget)
	{
		return false;
	}

	const TSharedRef<SSidebarDrawer> DrawerWidgetRef = InDrawer->DrawerWidget.ToSharedRef();

	if (ClosingDrawerWidgets.Contains(DrawerWidgetRef))
	{
		ClosingDrawerWidgets.Remove(DrawerWidgetRef);
	}
	else
	{
		const ESidebarTabLocation TabLocation = SidebarWidget->GetTabLocation();

		DrawersOverlay->AddSlot()
			.Padding(CalculateSlotMargin())
			.HAlign(SSidebarButton::GetHAlignFromTabLocation(TabLocation))
			.VAlign(SSidebarButton::GetVAlignFromTabLocation(TabLocation))
			[
				DrawerWidgetRef
			];
	}

	OpenDrawerWidgets.Add(DrawerWidgetRef);

	return true;
}

bool SSidebarContainer::RemoveDrawerOverlaySlot(const TSharedRef<FSidebarDrawer>& InDrawer, const bool bInAnimate)
{
	if (!InDrawer->DrawerWidget)
	{
		return false;
	}

	const TSharedRef<SSidebarDrawer> DrawerWidgetRef = InDrawer->DrawerWidget.ToSharedRef();

	if (bInAnimate)
	{
		ClosingDrawerWidgets.Add(DrawerWidgetRef);
	}
	else
	{
		ClosingDrawerWidgets.Remove(DrawerWidgetRef);

		DrawersOverlay->RemoveSlot(DrawerWidgetRef);
	}

	OpenDrawerWidgets.Remove(DrawerWidgetRef);

	return true;
}

void SSidebarContainer::CloseAllDrawerWidgets(const bool bInAnimate)
{
	for (const TSharedRef<FSidebarDrawer>& Drawer : SidebarWidget->GetAllDrawers())
	{
		CloseDrawer_Internal(Drawer, bInAnimate);
	}
}

EActiveTimerReturnType SSidebarContainer::OnOpenPendingDrawerTimer(const double InCurrentTime, const float InDeltaTime)
{
	if (const TSharedPtr<FSidebarDrawer> DrawerToOpen = PendingTabToOpen.Pin())
	{
		// Wait until the drawers overlay has been arranged once to open the drawer
		// It might not have geometry yet if we're adding back tabs on startup
		if (GetOverlaySize().IsZero())
		{
			return EActiveTimerReturnType::Continue;
		}

		OpenDrawer_Internal(DrawerToOpen.ToSharedRef(), bAnimatePendingTabOpen);
	}

	PendingTabToOpen.Reset();
	bAnimatePendingTabOpen = false;
	OpenPendingDrawerTimerHandle.Reset();

	return EActiveTimerReturnType::Stop;
}

void SSidebarContainer::OpenDrawerNextFrame(const TSharedRef<FSidebarDrawer>& InDrawer, const bool bInAnimate)
{
	if (InDrawer->DrawerWidget.IsValid() && OpenDrawerWidgets.Contains(InDrawer->DrawerWidget))
	{
		return;
	}

	PendingTabToOpen = InDrawer;
	bAnimatePendingTabOpen = bInAnimate;

	if (!OpenPendingDrawerTimerHandle.IsValid())
	{
		OpenPendingDrawerTimerHandle = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SSidebarContainer::OnOpenPendingDrawerTimer));
	}
}

FMargin SSidebarContainer::CalculateSlotMargin() const
{
	const FGeometry SidebarGeometry = SidebarWidget->GetTickSpaceGeometry();
	const float MinDrawerSize = SidebarGeometry.GetLocalSize().X - 4.f; // overlap with sidebar border slightly
	const FVector2D ShadowOffset(8.f, 8.f);
	const ESidebarTabLocation TabLocation = SidebarWidget->GetTabLocation();
	return FMargin(
		TabLocation == ESidebarTabLocation::Left ? MinDrawerSize : 0.f,
		-ShadowOffset.Y,
		TabLocation == ESidebarTabLocation::Right ? MinDrawerSize : 0.f,
		-ShadowOffset.Y);
}

void SSidebarContainer::CreateDrawerWidget(const TSharedRef<FSidebarDrawer>& InDrawer)
{
	// Calculate padding for the drawer itself
	const FGeometry SidebarGeometry = SidebarWidget->GetTickSpaceGeometry();
	const float MinDrawerSize = SidebarGeometry.GetLocalSize().X - 4.f; // overlap with sidebar border slightly
	const FMargin SlotPadding = CalculateSlotMargin();
	const float AvailableSize = GetOverlaySize().X - SlotPadding.GetTotalSpaceAlong<Orient_Horizontal>();
	const float MaxDrawerSize = AvailableSize * 0.5f; // max 50% of width or height
	const float TargetDrawerSize = AvailableSize * SidebarSizePercent;

	InDrawer->DrawerWidget =
		SNew(SSidebarDrawer, InDrawer, SidebarWidget->GetTabLocation())
		.MinDrawerSize(MinDrawerSize)
		.MaxDrawerSize(MaxDrawerSize)
		.TargetDrawerSize(TargetDrawerSize)
		.OnDrawerFocusLost(this, &SSidebarContainer::OnTabDrawerFocusLost)
		.OnOpenAnimationFinish(this, &SSidebarContainer::OnOpenAnimationFinish)
		.OnCloseAnimationFinish(this, &SSidebarContainer::OnCloseAnimationFinish)
		.OnDrawerSizeChanged(this, &SSidebarContainer::OnDrawerSizeChanged);
}

void SSidebarContainer::OpenDrawer_Internal(const TSharedRef<FSidebarDrawer>& InDrawer, const bool bInAnimate)
{
	if (InDrawer->DrawerWidget.IsValid() && OpenDrawerWidgets.Contains(InDrawer->DrawerWidget))
	{
		return;
	}

	for (const TSharedRef<FSidebarDrawer>& Drawer : SidebarWidget->GetAllDrawers())
	{
		CloseDrawer_Internal(Drawer, false, false);
	}

	PendingTabToOpen.Reset();
	bAnimatePendingTabOpen = false;

	// Skip creating a new widget if one is already animating closed
	if (!InDrawer->DrawerWidget.IsValid() || !ClosingDrawerWidgets.Contains(InDrawer->DrawerWidget))
	{
		CreateDrawerWidget(InDrawer);
	}
	AddDrawerOverlaySlot(InDrawer);

	InDrawer->DrawerWidget->Open(bInAnimate);
	InDrawer->bIsOpen = true;

	UpdateDrawerTabAppearance();

	// This changes the focus and will trigger focus-related events, such as closing other tabs,
	// so it's important that we only call it after we added the new drawer to OpenedDrawers.
	FSlateApplication::Get().SetKeyboardFocus(InDrawer->DrawerWidget);

	BroadcastDrawerOpenClose(InDrawer, /*bOpen=*/true);
}

void SSidebarContainer::CloseDrawer_Internal(const TSharedRef<FSidebarDrawer>& InDrawer, const bool bInAnimate
	, const bool bInSummonPinnedTabIfNothingOpened)
{
	const TSharedPtr<SSidebarDrawer> FoundDrawerWidget = FindOpenDrawerWidget(InDrawer);
	if (!FoundDrawerWidget.IsValid()
		|| !OpenDrawerWidgets.Contains(FoundDrawerWidget)
		|| ClosingDrawerWidgets.Contains(InDrawer->DrawerWidget))
	{
		return;
	}

	InDrawer->bIsOpen = false;

	RemoveDrawerOverlaySlot(InDrawer, bInAnimate);

	FoundDrawerWidget->Close(bInAnimate);

	UpdateDrawerTabAppearance();

	if (bInSummonPinnedTabIfNothingOpened)
	{
		SummonPinnedTabIfNothingOpened();
	}

	BroadcastDrawerOpenClose(InDrawer, /*bOpen=*/false);
}

void SSidebarContainer::SummonPinnedTabIfNothingOpened()
{
	// If there's already a drawer in the foreground, don't bring the pinned tab forward
	if (GetForegroundDrawer())
	{
		return;
	}

	// But if there's no current foreground tab, then bring forward a pinned tab (there should be at most one)
	// This should happen when:
	// - the current foreground tab is not pinned and loses focus
	// - the current foreground tab's drawer is manually closed by pressing on the tab button
	// - closing or restoring the current foreground tab
	if (const TSharedPtr<FSidebarDrawer> PinnedTab = FindFirstPinnedTab())
	{
		OpenDrawer_Internal(PinnedTab.ToSharedRef());
	}
}

void SSidebarContainer::UpdateDrawerTabAppearance()
{
	TSharedPtr<FSidebarDrawer> OpenedDrawer;
	if (OpenDrawerWidgets.Num() > 0)
	{
		OpenedDrawer = OpenDrawerWidgets.Last()->GetDrawer();
	}

	for (const TSharedRef<FSidebarDrawer>& Drawer : SidebarWidget->GetAllDrawers())
	{
		if (const TSharedPtr<SSidebarButton> TabButton = StaticCastSharedPtr<SSidebarButton>(Drawer->ButtonWidget))
		{
			TabButton->UpdateAppearance(OpenedDrawer);
		}
	}
}

void SSidebarContainer::DockDrawer_Internal(const TSharedRef<FSidebarDrawer>& InDrawer)
{
	for (const TSharedRef<FSidebarDrawer>& Drawer : SidebarWidget->GetAllDrawers())
	{
		CloseDrawer_Internal(Drawer, false);
	}

	InDrawer->bIsOpen = true;
	InDrawer->State.bIsPinned = false;
	InDrawer->State.bIsDocked = true;

	Reconstruct();

	UpdateDrawerTabAppearance();

	BroadcastDrawerOpenClose(InDrawer, /*bOpen=*/true);
}

void SSidebarContainer::UndockDrawer_Internal(const TSharedRef<FSidebarDrawer>& InDrawer)
{
	InDrawer->bIsOpen = false;
	InDrawer->State.bIsDocked = false;

	Reconstruct();

	UpdateDrawerTabAppearance();

	BroadcastDrawerOpenClose(InDrawer, /*bOpen=*/false);
}

TSharedPtr<SSidebarDrawer> SSidebarContainer::FindOpenDrawerWidget(const TSharedRef<FSidebarDrawer>& InDrawer) const
{
	const TSharedRef<SSidebarDrawer>* const OpenDrawWidget = OpenDrawerWidgets.FindByPredicate(
		[&InDrawer](const TSharedRef<SSidebarDrawer>& Drawer)
		{
			return InDrawer == Drawer->GetDrawer();
		});
	return OpenDrawWidget ? OpenDrawWidget->ToSharedPtr(): nullptr;
}

FName SSidebarContainer::GetOpenedDrawerId() const
{
	if (OpenDrawerWidgets.IsEmpty())
	{
		return NAME_None;
	}
	
	const TSharedRef<SSidebarDrawer> LastOpenDrawerWidget = OpenDrawerWidgets.Last();
	return LastOpenDrawerWidget->GetDrawer()->GetUniqueId();
}

TSharedPtr<FSidebarDrawer> SSidebarContainer::GetForegroundDrawer() const
{
	const int32 Index = OpenDrawerWidgets.FindLastByPredicate(
		[](const TSharedRef<SSidebarDrawer>& InDrawerWidget)
		{
			return InDrawerWidget->IsOpen() && !InDrawerWidget->IsClosing();
		});
	return Index == INDEX_NONE ? nullptr : OpenDrawerWidgets[Index]->GetDrawer();
}

void SSidebarContainer::OnTabDrawerFocusLost(const TSharedRef<SSidebarDrawer>& InDrawerWidget)
{
	const TSharedPtr<FSidebarDrawer> Drawer = InDrawerWidget->GetDrawer();
	if (!Drawer.IsValid())
	{
		return;
	}

	// Update to remove the focus marker
	UpdateDrawerTabAppearance();

	if (Drawer->State.bIsPinned)
	{
		return;
	}

	CloseDrawer_Internal(Drawer.ToSharedRef());
}

void SSidebarContainer::OnOpenAnimationFinish(const TSharedRef<SSidebarDrawer>& InDrawerWidget)
{
}

void SSidebarContainer::OnCloseAnimationFinish(const TSharedRef<SSidebarDrawer>& InDrawerWidget)
{
	RemoveDrawerOverlaySlot(InDrawerWidget->GetDrawer().ToSharedRef(), false);
}

void SSidebarContainer::OnDrawerSizeChanged(const TSharedRef<SSidebarDrawer>& InDrawerWidget, const float InNewPixelSize)
{
	if (!DrawersOverlay.IsValid())
	{
		return;
	}

	const TSharedPtr<FSidebarDrawer> Drawer = InDrawerWidget->GetDrawer();
	if (!Drawer.IsValid())
	{
		return;
	}

	const float DrawerOverlayWidth = GetOverlaySize().X;
	const float FillPercent = InNewPixelSize / DrawerOverlayWidth;
	SidebarSizePercent = FillPercent;

	SidebarWidget->OnStateChanged.ExecuteIfBound(SidebarWidget->GetState());
}

TSharedPtr<FSidebarDrawer> SSidebarContainer::FindDrawer(const FName InDrawerId) const
{
	const TSharedRef<FSidebarDrawer>* const FoundDrawer = SidebarWidget->GetAllDrawers().FindByPredicate(
		[InDrawerId](const TSharedRef<FSidebarDrawer>& InDrawer)
		{
			return InDrawerId == InDrawer->GetUniqueId();
		});
	return FoundDrawer ? *FoundDrawer : TSharedPtr<FSidebarDrawer>();
}

TSharedPtr<FSidebarDrawer> SSidebarContainer::FindFirstPinnedTab() const
{
	for (const TSharedRef<FSidebarDrawer>& Drawer : SidebarWidget->GetAllDrawers())
	{
		if (Drawer->State.bIsPinned)
		{
			return Drawer;
		}
	}
	return nullptr;
}

void SSidebarContainer::OnContentSlotResizing(const float InFillPercent)
{
	ContentSizePercent = InFillPercent;
}

void SSidebarContainer::OnSidebarSlotResizing(const float InFillPercent)
{
	if (SidebarSizePercent < FSidebarState::AutoDockThresholdSize)
	{
		FSlateApplication::Get().ReleaseAllPointerCapture();

		SidebarWidget->UndockAllDrawers();

		bWantsToAutoDock = true;
		ContentSizePercent = ContentSizeBeforeResize;
		SidebarSizePercent = SidebarSizeBeforeResize;

		FSidebarState NewState = SidebarWidget->GetState();
		NewState.SetDrawerSizes(SidebarSizePercent, ContentSizePercent);
		SidebarWidget->OnStateChanged.ExecuteIfBound(NewState);
	}
	else
	{
		// Save the current size when we start resizing
		if (SidebarSizeBeforeResize == 0.f)
		{
			ContentSizeBeforeResize = ContentSizePercent;
			SidebarSizeBeforeResize = SidebarSizePercent;
		}

		SidebarSizePercent = InFillPercent;
	}
}

void SSidebarContainer::OnSplitterResized()
{
	bWantsToAutoDock = false;
	ContentSizeBeforeResize = 0.f;
	SidebarSizeBeforeResize = 0.f;

	SidebarWidget->OnStateChanged.ExecuteIfBound(SidebarWidget->GetState());
}

void SSidebarContainer::BroadcastDrawerOpenClose(const TSharedRef<FSidebarDrawer>& InDrawer, const bool bOpen)
{
	InDrawer->OnDrawerOpenCloseDelegate.ExecuteIfBound(InDrawer->GetUniqueId(), bOpen);

	TArray<TSharedRef<ISidebarDrawerContent>> ContentSections;
	InDrawer->ContentSections.GenerateValueArray(ContentSections);

	for (const TSharedRef<ISidebarDrawerContent>& DrawerSection : ContentSections)
	{
		DrawerSection->OnDrawerOpenClose(bOpen);
	}
}

#undef LOCTEXT_NAMESPACE
