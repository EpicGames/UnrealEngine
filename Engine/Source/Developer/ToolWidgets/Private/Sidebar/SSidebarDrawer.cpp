// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSidebarDrawer.h"
#include "Framework/Application/SlateApplication.h"
#include "Sidebar/SidebarDrawer.h"
#include "Sidebar/SSidebar.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBox.h"

SSidebarDrawer::~SSidebarDrawer()
{
	FSlateApplication::Get().OnFocusChanging().RemoveAll(this);

	FSlateThrottleManager::Get().LeaveResponsiveMode(AnimationThrottle);
	FSlateThrottleManager::Get().LeaveResponsiveMode(ResizeThrottleHandle);
}

void SSidebarDrawer::Construct(const FArguments& InArgs, const TSharedRef<FSidebarDrawer>& InDrawer, const ESidebarTabLocation InTabLocation)
{
	check(InDrawer->ContentWidget.IsValid());

	DrawerWeak = InDrawer;
	TabLocation = InTabLocation;

	MinDrawerSize = InArgs._MinDrawerSize;
	MaxDrawerSize = InArgs._MaxDrawerSize;
	TargetDrawerSize = FMath::Clamp(InArgs._TargetDrawerSize, MinDrawerSize, MaxDrawerSize);
	ShadowOffset = InArgs._ShadowOffset;
	ExpanderHandleSize = InArgs._ExpanderHandleSize;
	
	OnDrawerSizeChanged = InArgs._OnDrawerSizeChanged;
	OnDrawerFocused = InArgs._OnDrawerFocused;
	OnDrawerFocusLost = InArgs._OnDrawerFocusLost;
	OnCloseAnimationFinish = InArgs._OnCloseAnimationFinish;
	OnOpenAnimationFinish = InArgs._OnOpenAnimationFinish;

	OpenCloseAnimation = FCurveSequence(0.f, AnimationLength, ECurveEaseFunction::QuadOut);

	FSlateApplication::Get().OnFocusChanging().AddSP(this, &SSidebarDrawer::OnGlobalFocusChanging);

	ChildSlot
	[
		SNew(SBox)
		.Clipping(EWidgetClipping::ClipToBounds)
		[
			InDrawer->ContentWidget.ToSharedRef()
		]
	];
}

void SSidebarDrawer::SetCurrentSize(const float InSize)
{
	CurrentSize = FMath::Clamp(InSize, MinDrawerSize, TargetDrawerSize);
}

void SSidebarDrawer::Open(const bool bInAnimateOpen)
{
	if (!bInAnimateOpen)
	{
		SetCurrentSize(TargetDrawerSize);
		OpenCloseAnimation.JumpToEnd();
		return;
	}

	if (OpenCloseAnimation.IsInReverse())
	{
		OpenCloseAnimation.Reverse();
	}

	OpenCloseAnimation.Play(AsShared(), false, OpenCloseAnimation.GetSequenceTime(), false);

	if (!OpenCloseTimer.IsValid())
	{
		AnimationThrottle = FSlateThrottleManager::Get().EnterResponsiveMode();
		OpenCloseTimer = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SSidebarDrawer::UpdateAnimation));
	}
}

void SSidebarDrawer::Close(const bool bInAnimateOpen)
{
	if (!bInAnimateOpen)
	{
		SetCurrentSize(0.f);
		OpenCloseAnimation.JumpToStart();
		return;
	}

	if (OpenCloseAnimation.IsForward())
	{
		OpenCloseAnimation.Reverse();
	}

	if (!OpenCloseTimer.IsValid())
	{
		AnimationThrottle = FSlateThrottleManager::Get().EnterResponsiveMode();
		OpenCloseTimer = RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SSidebarDrawer::UpdateAnimation));
	}
}

bool SSidebarDrawer::IsOpen() const
{
	return !OpenCloseAnimation.IsAtStart();
}

bool SSidebarDrawer::IsClosing() const
{
	return OpenCloseAnimation.IsPlaying() && OpenCloseAnimation.IsInReverse();
}

TSharedPtr<FSidebarDrawer> SSidebarDrawer::GetDrawer() const
{
	return DrawerWeak.Pin();
}

bool SSidebarDrawer::SupportsKeyboardFocus() const 
{
	return true;
}

FVector2D SSidebarDrawer::ComputeDesiredSize(const float InLayoutScaleMultiplier) const
{
	switch (TabLocation)
	{
	case ESidebarTabLocation::Left:
	case ESidebarTabLocation::Right:
		return FVector2D(TargetDrawerSize + ShadowOffset.X, 1.f);
	case ESidebarTabLocation::Top:
	case ESidebarTabLocation::Bottom:
		return FVector2D(1.f, TargetDrawerSize + ShadowOffset.Y);
	}
	return FVector2D::One();
}

void SSidebarDrawer::OnArrangeChildren(const FGeometry& InAllottedGeometry, FArrangedChildren& ArrangedChildren) const 
{
	const EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();
	if (!ArrangedChildren.Accepts(ChildVisibility))
	{
		return;
	}

	FVector2f ChildOffset;
	FVector2f LocalSize;

	switch (TabLocation)
	{
	case ESidebarTabLocation::Left:
		ChildOffset = FVector2f(0.f, ShadowOffset.Y);
		LocalSize = FVector2f(TargetDrawerSize, InAllottedGeometry.GetLocalSize().Y - (ShadowOffset.Y * 2));
		break;
	case ESidebarTabLocation::Right:
		ChildOffset = ShadowOffset;
		LocalSize = FVector2f(TargetDrawerSize, InAllottedGeometry.GetLocalSize().Y - (ShadowOffset.Y * 2));
		break;
	case ESidebarTabLocation::Top:
		ChildOffset = FVector2f(ShadowOffset.X, 0.f);
		LocalSize = FVector2f(InAllottedGeometry.GetLocalSize().X - (ShadowOffset.X * 2), TargetDrawerSize);
		break;
	case ESidebarTabLocation::Bottom:
		ChildOffset = ShadowOffset;
		LocalSize = FVector2f(InAllottedGeometry.GetLocalSize().X - (ShadowOffset.X * 2), TargetDrawerSize);
		break;
	}

	ArrangedChildren.AddWidget(InAllottedGeometry.MakeChild(ChildSlot.GetWidget(), ChildOffset, LocalSize));
}

FReply SSidebarDrawer::OnMouseButtonDown(const FGeometry& InAllottedGeometry, const FPointerEvent& InMouseEvent)
{
	FReply Reply = FReply::Unhandled();

	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		const FGeometry ResizeHandleGeometry = GetResizeHandleGeometry(InAllottedGeometry);

		if (ResizeHandleGeometry.IsUnderLocation(InMouseEvent.GetScreenSpacePosition()))
		{
			bIsResizing = true;
			InitialResizeGeometry = ResizeHandleGeometry;
			InitialSizeAtResize = CurrentSize;
			ResizeThrottleHandle = FSlateThrottleManager::Get().EnterResponsiveMode();

			Reply = FReply::Handled().CaptureMouse(SharedThis(this));
		}
	}

	return Reply;
}

FReply SSidebarDrawer::OnMouseButtonUp(const FGeometry& InAllottedGeometry, const FPointerEvent& InMouseEvent)
{
	if (bIsResizing && InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bIsResizing = false;
		FSlateThrottleManager::Get().LeaveResponsiveMode(ResizeThrottleHandle);

		OnDrawerSizeChanged.ExecuteIfBound(SharedThis(this), TargetDrawerSize);

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

FReply SSidebarDrawer::OnMouseMove(const FGeometry& InAllottedGeometry, const FPointerEvent& InMouseEvent)
{
	const FGeometry ResizeHandleGeometry = GetResizeHandleGeometry(InAllottedGeometry);

	bIsResizeHandleHovered = ResizeHandleGeometry.IsUnderLocation(InMouseEvent.GetScreenSpacePosition());

	if (bIsResizing && this->HasMouseCapture() && !InMouseEvent.GetCursorDelta().IsZero())
	{
		const FVector2f MousePosition = InMouseEvent.GetScreenSpacePosition();
		const FVector2f LocalMousePosition = InitialResizeGeometry.AbsoluteToLocal(MousePosition);

		float DeltaSize = 0.f;

		switch (TabLocation)
		{
		case ESidebarTabLocation::Left:
			DeltaSize = LocalMousePosition.X;
			break;
		case ESidebarTabLocation::Right:
			DeltaSize = -LocalMousePosition.X;
			break;
		case ESidebarTabLocation::Top:
			DeltaSize = LocalMousePosition.Y;
			break;
		case ESidebarTabLocation::Bottom:
			DeltaSize = -LocalMousePosition.Y;
			break;
		}

		TargetDrawerSize = FMath::Clamp(InitialSizeAtResize + DeltaSize, MinDrawerSize, MaxDrawerSize);
		SetCurrentSize(InitialSizeAtResize + DeltaSize);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SSidebarDrawer::OnMouseLeave(const FPointerEvent& InMouseEvent)
{
	SCompoundWidget::OnMouseLeave(InMouseEvent);

	bIsResizeHandleHovered = false;
}

FCursorReply SSidebarDrawer::OnCursorQuery(const FGeometry& InGeometry, const FPointerEvent& InCursorEvent) const
{
	if (bIsResizing || bIsResizeHandleHovered)
	{
		switch (TabLocation)
		{
		case ESidebarTabLocation::Left:
		case ESidebarTabLocation::Right:
			return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
		case ESidebarTabLocation::Top:
		case ESidebarTabLocation::Bottom:
			return FCursorReply::Cursor(EMouseCursor::ResizeUpDown);
		}
	}

	return FCursorReply::Unhandled();
}

int32 SSidebarDrawer::OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InCullingRect
	, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const
{
	static const FSlateBrush* BackgroundBrush = FAppStyle::Get().GetBrush(TEXT("Docking.Sidebar.DrawerBackground"));
	static const FSlateBrush* ShadowBrush = FAppStyle::Get().GetBrush(TEXT("Docking.Sidebar.DrawerShadow"));
	static const FSlateBrush* BorderBrush = FAppStyle::Get().GetBrush(TEXT("Docking.Sidebar.Border"));
	static const FSlateBrush* BorderSquareEdgeBrush;
	switch (TabLocation)
	{
	case ESidebarTabLocation::Left:
		BorderSquareEdgeBrush = FAppStyle::Get().GetBrush(TEXT("Docking.Sidebar.Border_SquareLeft"));
		break;
	case ESidebarTabLocation::Right:
		BorderSquareEdgeBrush = FAppStyle::Get().GetBrush(TEXT("Docking.Sidebar.Border_SquareRight"));
		break;
	case ESidebarTabLocation::Top:
	case ESidebarTabLocation::Bottom:
		// @TODO: There are no existing top or bottom brushes
		BorderSquareEdgeBrush = BorderBrush;
		break;
	}
	static const FSplitterStyle* SplitterStyle = &FAppStyle::Get().GetWidgetStyle<FSplitterStyle>(TEXT("Splitter"));
	static const FSlateColor ShadowColor = FAppStyle::Get().GetSlateColor(TEXT("Colors.Foldout"));

	const FGeometry RenderTransformedChildGeometry = GetRenderTransformedGeometry(InAllottedGeometry);
	const FGeometry ResizeHandleGeometry = GetResizeHandleGeometry(InAllottedGeometry);
	const FVector2f LocalSize = InAllottedGeometry.GetLocalSize();

	FVector2f ContentsLocalOrigin;
	FVector2f ContentsLocalSize;

	switch (TabLocation)
	{
	case ESidebarTabLocation::Left:
		ContentsLocalOrigin = FVector2f(0.f, ShadowOffset.Y);
		ContentsLocalSize = FVector2f(TargetDrawerSize, LocalSize.Y - (ShadowOffset.Y * 2));
		break;
	case ESidebarTabLocation::Right:
		ContentsLocalOrigin = ShadowOffset;
		ContentsLocalSize = FVector2f(TargetDrawerSize, LocalSize.Y - (ShadowOffset.Y * 2));
		break;
	case ESidebarTabLocation::Top:
	case ESidebarTabLocation::Bottom:
		ContentsLocalOrigin = ShadowOffset;
		ContentsLocalSize = FVector2f(LocalSize.X - (ShadowOffset.X * 2), TargetDrawerSize);
		break;
	}

	const FPaintGeometry OffsetPaintGeom = RenderTransformedChildGeometry.ToPaintGeometry(ContentsLocalSize, FSlateLayoutTransform(ContentsLocalOrigin));

	// Draw the resize handle
	if (bIsResizing || bIsResizeHandleHovered)
	{
		const FSlateBrush* SplitterBrush = &SplitterStyle->HandleHighlightBrush;
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			InLayerId,
			ResizeHandleGeometry.ToPaintGeometry(),
			SplitterBrush,
			ESlateDrawEffect::None,
			SplitterBrush->GetTint(InWidgetStyle));
	}

	// Main Shadow
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		InLayerId++,
		RenderTransformedChildGeometry.ToPaintGeometry(),
		ShadowBrush,
		ESlateDrawEffect::None,
		ShadowBrush->GetTint(InWidgetStyle));

	// Background
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		InLayerId++,
		OffsetPaintGeom,
		BackgroundBrush,
		ESlateDrawEffect::None,
		BackgroundBrush->GetTint(InWidgetStyle));

	int32 OutLayerId = SCompoundWidget::OnPaint(InArgs, RenderTransformedChildGeometry, InCullingRect, OutDrawElements, InLayerId, InWidgetStyle, bInParentEnabled);

	TSharedPtr<SWidget> TabButton;
	if (const TSharedPtr<FSidebarDrawer> Drawer = DrawerWeak.Pin())
	{
		TabButton = Drawer->ButtonWidget;
	}
	if (!TabButton.IsValid())
	{
		return ++OutLayerId;
	}

	// Top border
	switch (TabLocation)
	{
	case ESidebarTabLocation::Left:
	case ESidebarTabLocation::Right:
		{
			// Example of how border box is drawn with the tab notch cut out on the right side
			// (OpenDirection == ESidebarTabLocation::Right)
			//
			//                    + - - - - - - +
			//                    : /---------\ :
			// ClipAboveTabButton : |         | : 
			//                    : |         | :
			//            TabTopY + - - - - - - +
			//                    : |           :  |
			// ClipAtTabButton    : |           :  |  (right edge outside clip is clipped off)
			//                    : |           :  |
			//         TabBottomY + - - - - - - +
			//                    : |         | :
			// ClipBelowTabButton : |         | :
			//                    : \---------/ :
			//                    + - - - - - - +
			//                                  <-->
			//                              NotchOffset
			//
			// Originally, I tried making the middle clip region thinner (to clip out the notch)
			// while keeping the geometry identical, but this looks worse when the tab notch needs to
			// be at the top or bottom, since the top/bottom edge of the border wouldn't extend all the
			// way to the edge.
			const FGeometry TabButtonGeometry = TabButton->GetPaintSpaceGeometry();

			// Compute the top/bottom of the tab in our local space.
			const float BorderWidth = BorderBrush->OutlineSettings.Width;
			const float TabTopY = RenderTransformedChildGeometry.AbsoluteToLocal(TabButtonGeometry.GetAbsolutePositionAtCoordinates(FVector2f::ZeroVector)).Y + 0.5f * BorderWidth;
			const float TabBottomY = RenderTransformedChildGeometry.AbsoluteToLocal(TabButtonGeometry.GetAbsolutePositionAtCoordinates(FVector2f::UnitVector)).Y - 0.5f * BorderWidth;

			// Create the geometry for the notched portion, where one edge extends past the clipping rect.
			const FVector2f NotchOffsetSize(TabButtonGeometry.GetLocalSize().X, 0.f);
			const FVector2f NotchOffsetTranslate = TabLocation == ESidebarTabLocation::Left ? -NotchOffsetSize : FVector2f::ZeroVector;
			const FPaintGeometry NotchOffsetPaintGeom = RenderTransformedChildGeometry.ToPaintGeometry(ContentsLocalSize + NotchOffsetSize
				, FSlateLayoutTransform(ContentsLocalOrigin + NotchOffsetTranslate));

			// Split the border box into three clipping zones.
			const FPaintGeometry ClipAboveTabButton = RenderTransformedChildGeometry.ToPaintGeometry(FVector2f(LocalSize.X, TabTopY)
				, FSlateLayoutTransform(FVector2f(0.f, 0.f)));
			const FPaintGeometry ClipAtTabButton = RenderTransformedChildGeometry.ToPaintGeometry(FVector2f(LocalSize.X, TabBottomY - TabTopY)
				, FSlateLayoutTransform(FVector2f(0.f, TabTopY)));
			const FPaintGeometry ClipBelowTabButton = RenderTransformedChildGeometry.ToPaintGeometry(FVector2f(LocalSize.X, LocalSize.Y - TabBottomY)
				, FSlateLayoutTransform(FVector2f(0.f, TabBottomY)));

			// If the tab button touches a corner on the edge of the border, switch the brush to
			// draw that corner squared-off. When a tab is near the very top or bottom of its sidebar,
			// this makes the outline look slightly nicer and more connected.
			const int32 UpperCornerIndex = TabLocation == ESidebarTabLocation::Left ? 0 : 1;
			const int32 LowerCornerIndex = TabLocation == ESidebarTabLocation::Left ? 3 : 2;
			const bool bTabTouchesUpperCorner = TabTopY < ShadowOffset.Y + BorderBrush->OutlineSettings.CornerRadii[UpperCornerIndex];
			const bool bTabTouchesLowerCorner = TabBottomY > LocalSize.Y - ShadowOffset.Y - BorderBrush->OutlineSettings.CornerRadii[LowerCornerIndex];
			const FSlateBrush* AboveTabBrush = bTabTouchesUpperCorner ? BorderSquareEdgeBrush : BorderBrush;
			const FSlateBrush* BelowTabBrush = bTabTouchesLowerCorner ? BorderSquareEdgeBrush : BorderBrush;

			// Draw portion above the tab
			OutDrawElements.PushClip(FSlateClippingZone(ClipAboveTabButton));
			FSlateDrawElement::MakeBox(OutDrawElements, OutLayerId, OffsetPaintGeom, AboveTabBrush
				, ESlateDrawEffect::None, AboveTabBrush->GetTint(InWidgetStyle));
			OutDrawElements.PopClip();

			// Draw "notched" portion next to the tab
			OutDrawElements.PushClip(FSlateClippingZone(ClipAtTabButton));
			FSlateDrawElement::MakeBox(OutDrawElements, OutLayerId, NotchOffsetPaintGeom, BorderSquareEdgeBrush
				, ESlateDrawEffect::None, BorderSquareEdgeBrush->GetTint(InWidgetStyle));
			OutDrawElements.PopClip();

			// Draw portion below the tab
			OutDrawElements.PushClip(FSlateClippingZone(ClipBelowTabButton));
			FSlateDrawElement::MakeBox(OutDrawElements, OutLayerId, OffsetPaintGeom, BelowTabBrush
				, ESlateDrawEffect::None, BelowTabBrush->GetTint(InWidgetStyle));
			OutDrawElements.PopClip();
		}
		break;
	case ESidebarTabLocation::Top:
	case ESidebarTabLocation::Bottom:
		{
			// When opened from the bottom, draw the full border.
			// Cutting out the "notch" for the corresponding tab is only supported in left/right orientations.
			FSlateDrawElement::MakeBox(OutDrawElements, OutLayerId, OffsetPaintGeom, BorderBrush
				, ESlateDrawEffect::None, BorderBrush->GetTint(InWidgetStyle));
		}
		break;
	}

	return ++OutLayerId;
}

FGeometry SSidebarDrawer::GetRenderTransformedGeometry(const FGeometry& InAllottedGeometry) const
{
	switch (TabLocation)
	{
	case ESidebarTabLocation::Left:
		return InAllottedGeometry.MakeChild(FSlateRenderTransform(FVector2f(CurrentSize - TargetDrawerSize, 0.f)));
	case ESidebarTabLocation::Right:
		return InAllottedGeometry.MakeChild(FSlateRenderTransform(FVector2f(TargetDrawerSize - CurrentSize, 0.f)));
	case ESidebarTabLocation::Top:
		return InAllottedGeometry.MakeChild(FSlateRenderTransform(FVector2f(0.f, CurrentSize - TargetDrawerSize)));
	case ESidebarTabLocation::Bottom:
		return InAllottedGeometry.MakeChild(FSlateRenderTransform(FVector2f(0.f, TargetDrawerSize - CurrentSize)));
	}
	return FGeometry();
}

FGeometry SSidebarDrawer::GetResizeHandleGeometry(const FGeometry& InAllottedGeometry) const
{
	const FGeometry RenderTransformedGeometry = GetRenderTransformedGeometry(InAllottedGeometry);

	FVector2f LocalSize;
	FVector2f Translation;

	switch (TabLocation)
	{
	case ESidebarTabLocation::Left:
		LocalSize = FVector2f(ExpanderHandleSize, InAllottedGeometry.GetLocalSize().Y - (ShadowOffset.Y * 2));
		Translation = FVector2f(RenderTransformedGeometry.GetLocalSize().X - ShadowOffset.X, ShadowOffset.Y);
		break;
	case ESidebarTabLocation::Right:
		LocalSize = FVector2f(ExpanderHandleSize, InAllottedGeometry.GetLocalSize().Y - (ShadowOffset.Y * 2));
		Translation = ShadowOffset - FVector2f(ExpanderHandleSize, 0.f);
		break;
	case ESidebarTabLocation::Top:
		LocalSize = FVector2f(InAllottedGeometry.GetLocalSize().X - (ShadowOffset.X * 2), ExpanderHandleSize);
		Translation = FVector2f(ShadowOffset.X, RenderTransformedGeometry.GetLocalSize().Y - ShadowOffset.Y);
		break;
	case ESidebarTabLocation::Bottom:
		LocalSize = FVector2f(InAllottedGeometry.GetLocalSize().X - (ShadowOffset.X * 2), ExpanderHandleSize);
		Translation = ShadowOffset - FVector2f(0.f, ExpanderHandleSize);
		break;
	}

	return RenderTransformedGeometry.MakeChild(LocalSize, FSlateLayoutTransform(Translation));
}

EActiveTimerReturnType SSidebarDrawer::UpdateAnimation(const double InCurrentTime, const float InDeltaTime)
{
	SetCurrentSize(FMath::Lerp(0.f, TargetDrawerSize, OpenCloseAnimation.GetLerp()));

	if (!OpenCloseAnimation.IsPlaying())
	{
		if (OpenCloseAnimation.IsAtStart())
		{
			OnCloseAnimationFinish.ExecuteIfBound(SharedThis(this));
		}
		else if (OpenCloseAnimation.IsAtEnd())
		{
			OnOpenAnimationFinish.ExecuteIfBound(SharedThis(this));
		}

		FSlateThrottleManager::Get().LeaveResponsiveMode(AnimationThrottle);
		OpenCloseTimer.Reset();

		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}

static bool IsLegalWidgetFocused(const FWidgetPath& InFocusPath, const TArrayView<TSharedRef<SWidget>>& InLegalFocusWidgets)
{
	for (const TSharedRef<SWidget>& Widget : InLegalFocusWidgets)
	{
		if (InFocusPath.ContainsWidget(&Widget.Get()))
		{
			return true;
		}
	}

	return false;
}

void SSidebarDrawer::OnGlobalFocusChanging(const FFocusEvent& InFocusEvent
	, const FWeakWidgetPath& InOldFocusedWidgetPath, const TSharedPtr<SWidget>& InOldFocusedWidget
	, const FWidgetPath& InNewFocusedWidgetPath, const TSharedPtr<SWidget>& InNewFocusedWidget)
{
	// Sometimes when dismissing focus can change which will trigger this again
	static bool bIsReEntrant = false;
	if (bIsReEntrant)
	{
		return;
	}
	TGuardValue<bool> ReEntrancyGuard(bIsReEntrant, true);

	const TSharedPtr<FSidebarDrawer> Drawer = DrawerWeak.Pin();

	// Only open drawers that are not docked or pinned need to close the drawer when focus is lost
	if (!Drawer || Drawer->State.bIsDocked || Drawer->State.bIsPinned || !Drawer->bIsOpen)
	{
		return;
	}

	// Do not close due to slow tasks as those opening send window activation events
	if (GIsSlowTask || FSlateApplication::Get().GetActiveModalWindow().IsValid())
	{
		return;
	}

	const TSharedRef<SSidebarDrawer> ThisWidget = SharedThis(this);

	TArray<TSharedRef<SWidget>, TInlineAllocator<4>> LegalFocusWidgets;
	LegalFocusWidgets.Add(ThisWidget);
	LegalFocusWidgets.Add(ChildSlot.GetWidget());
	LegalFocusWidgets.Add(Drawer->ButtonWidget.ToSharedRef());

	bool bShouldLoseFocus = false;

	if (IsLegalWidgetFocused(InNewFocusedWidgetPath, MakeArrayView(LegalFocusWidgets)))
	{
		// New focus is on this tab, so make it active
		if (!IsClosing())
		{
			OnDrawerFocused.ExecuteIfBound(ThisWidget);
		}
	}
	else if (InNewFocusedWidgetPath.IsValid())
	{
		// New focus is on something else, try to check if it's a menu or child window
		const TSharedRef<SWindow> NewWindow = InNewFocusedWidgetPath.GetWindow();
		const TSharedPtr<SWindow> ThisWindow = FSlateApplication::Get().FindWidgetWindow(ThisWidget);

		// See if this is a child window (like a color picker being opened from details), and if so, don't dismiss
		if (!NewWindow->IsDescendantOf(ThisWindow))
		{
			if (const TSharedPtr<SWidget> MenuHost = FSlateApplication::Get().GetMenuHostWidget())
			{
				FWidgetPath MenuHostPath;

				// See if the menu being opened is owned by the drawer contents and if so the menu should not be dismissed
				FSlateApplication::Get().GeneratePathToWidgetUnchecked(MenuHost.ToSharedRef(), MenuHostPath);

				if (!MenuHostPath.ContainsWidget(&ChildSlot.GetWidget().Get()))
				{
					bShouldLoseFocus = true;
				}
			}
			else
			{
				bShouldLoseFocus = true;
			}
		}
	}
	else
	{
		bShouldLoseFocus = true;
	}

	if (bShouldLoseFocus)
	{
		OnDrawerFocusLost.ExecuteIfBound(ThisWidget);
	}
}
