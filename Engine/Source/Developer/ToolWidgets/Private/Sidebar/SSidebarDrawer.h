// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SSidebarButton.h"
#include "Animation/CurveSequence.h"
#include "Application/ThrottleManager.h"
#include "Framework/SlateDelegates.h"
#include "Layout/Geometry.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FSidebarDrawer;
class ISidebarDrawerContent;
class SSidebarDrawer;
enum class ESidebarTabLocation : uint8;
struct FSplitterStyle;

DECLARE_DELEGATE_OneParam(FGenericSidebarDrawerWidgetDelegate, const TSharedRef<SSidebarDrawer>&);
DECLARE_DELEGATE_TwoParams(FOnSidebarDrawerTargetSizeChanged, const TSharedRef<SSidebarDrawer>&, float);

/** Handles sliding drawer animation */
class SSidebarDrawer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSidebarDrawer)
		: _ShadowOffset(FVector2f(8.f, 8.f))
		, _ExpanderHandleSize(5.f)
	{
		_Clipping = EWidgetClipping::ClipToBounds;
	}
		/** The minimum size a drawer can be when opened. This unit is in window space */
		SLATE_ARGUMENT(float, MinDrawerSize)
		/** The maximum size a drawer can be when opened. This unit is in window space */
		SLATE_ARGUMENT(float, MaxDrawerSize)
		/** The size that the drawer should be, clamped to the above min/max values. This unit is in window space */
		SLATE_ARGUMENT(float, TargetDrawerSize)
		/** The side of the drop shadow surrounding the drawer */
		SLATE_ARGUMENT(FDeprecateSlateVector2D, ShadowOffset)
		/** The size of the handle used to resize the drawer */
		SLATE_ARGUMENT(float, ExpanderHandleSize)
		/** Called when the drawer size is changed by the user */
		SLATE_EVENT(FOnSidebarDrawerTargetSizeChanged, OnDrawerSizeChanged)
		/** Called when the drawer gains focus */
		SLATE_EVENT(FGenericSidebarDrawerWidgetDelegate, OnDrawerFocused)
		/** Called when the drawer loses focus */
		SLATE_EVENT(FGenericSidebarDrawerWidgetDelegate, OnDrawerFocusLost)
		/** Called when the drawer is completely opened (i.e will be called once the open animation completes */
		SLATE_EVENT(FGenericSidebarDrawerWidgetDelegate, OnOpenAnimationFinish)
		/** Called when the drawer is completely closed (i.e will be called once the close animation completes */
		SLATE_EVENT(FGenericSidebarDrawerWidgetDelegate, OnCloseAnimationFinish)
	SLATE_END_ARGS()

	~SSidebarDrawer();

	void Construct(const FArguments& InArgs, const TSharedRef<FSidebarDrawer>& InDrawer, const ESidebarTabLocation InTabLocation);

	/**
	 * Opens the drawer.
	 *
	 * @param bInAnimateOpen Whether to play an animation when opening the drawer, defaults to true.
	 */
	void Open(const bool bInAnimateOpen = true);

	/**
	 * Closes the drawer.
	 *
	 * @param bInAnimateOpen Whether to play an animation when closing the drawer, defaults to true.
	 */
	void Close(const bool bInAnimateOpen = true);

	/** @return True if the drawer is open. */
	bool IsOpen() const;

	/** @return True if the drawer is currently playing the close animation. */
	bool IsClosing() const;

	/** Sets the current size of the drawer in pixels, ignoring any open/close animation. */
	void SetCurrentSize(const float InSize);

	/** @return The drawer associated with this drawer widget. */
	TSharedPtr<FSidebarDrawer> GetDrawer() const;

protected:
	static constexpr float AnimationLength = 0.15f;

	//~ Begin SWidget
	virtual bool SupportsKeyboardFocus() const override;
	virtual FVector2D ComputeDesiredSize(const float InLayoutScaleMultiplier) const override;
	virtual void OnArrangeChildren(const FGeometry& InAllottedGeometry, FArrangedChildren& ArrangedChildren) const override;
	virtual FReply OnMouseButtonDown(const FGeometry& InAllottedGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& InAllottedGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& InAllottedGeometry, const FPointerEvent& InMouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& InMouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& InGeometry, const FPointerEvent& InCursorEvent) const override;
	virtual int32 OnPaint(const FPaintArgs& InArgs, const FGeometry& InAllottedGeometry, const FSlateRect& InCullingRect
		, FSlateWindowElementList& OutDrawElements, int32 InLayerId, const FWidgetStyle& InWidgetStyle, bool bInParentEnabled) const override;
	//~ End SWidget

	FGeometry GetRenderTransformedGeometry(const FGeometry& InAllottedGeometry) const;

	FGeometry GetResizeHandleGeometry(const FGeometry& InAllottedGeometry) const;

	EActiveTimerReturnType UpdateAnimation(const double InCurrentTime, const float InDeltaTime);

	void OnGlobalFocusChanging(const FFocusEvent& InFocusEvent
		, const FWeakWidgetPath& InOldFocusedWidgetPath, const TSharedPtr<SWidget>& InOldFocusedWidget
		, const FWidgetPath& InNewFocusedWidgetPath, const TSharedPtr<SWidget>& InNewFocusedWidget);

	TWeakPtr<FSidebarDrawer> DrawerWeak;
	ESidebarTabLocation TabLocation = ESidebarTabLocation::Right;

	float MinDrawerSize = 0.f;
	float MaxDrawerSize = 0.f;
	float TargetDrawerSize = 0.f;
	FVector2f ShadowOffset = FVector2f::ZeroVector;
	float ExpanderHandleSize = 5.f;

	FOnSidebarDrawerTargetSizeChanged OnDrawerSizeChanged;
	FGenericSidebarDrawerWidgetDelegate OnDrawerFocused;
	FGenericSidebarDrawerWidgetDelegate OnDrawerFocusLost;
	FGenericSidebarDrawerWidgetDelegate OnCloseAnimationFinish;
	FGenericSidebarDrawerWidgetDelegate OnOpenAnimationFinish;

	FCurveSequence OpenCloseAnimation;
	TSharedPtr<FActiveTimerHandle> OpenCloseTimer;

	FThrottleRequest ResizeThrottleHandle;
	FThrottleRequest AnimationThrottle;

	float CurrentSize = 0.f;

	bool bIsResizing = false;
	bool bIsResizeHandleHovered = false;
	float InitialSizeAtResize = 0.f;
	FGeometry InitialResizeGeometry;
};
