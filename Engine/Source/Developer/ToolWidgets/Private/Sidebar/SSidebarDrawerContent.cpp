// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSidebarDrawerContent.h"
#include "Framework/Application/SlateApplication.h"
#include "Sidebar/ISidebarDrawerContent.h"
#include "Sidebar/SidebarDrawer.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SSidebarDrawerContent"

void SSidebarDrawerContent::Construct(const FArguments& InArgs, const TWeakPtr<FSidebarDrawer>& InOwnerDrawerWeak)
{
	OwnerDrawerWeak = InOwnerDrawerWeak;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		[
			SNew(SBox)
			.Visibility_Lambda([this]()
				{
					return GetOrderedSections().Num() > 1 ? EVisibility::Visible : EVisibility::Collapsed;
				})
			[
				SNew(SBox)
				.Padding(FMargin(0.f, 4.f))
				[
					SAssignNew(ButtonBox, SWrapBox)
					.HAlign(HAlign_Center)
					.UseAllottedSize(true)
					.InnerSlotPadding(FVector2D(4.f, 4.f))
				]
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		.VAlign(VAlign_Fill)
		[
			SAssignNew(ContentBox, SVerticalBox)
		]
	];

	BuildContent();
}

void SSidebarDrawerContent::BuildContent()
{
	const TSharedPtr<FSidebarDrawer> Drawer = OwnerDrawerWeak.Pin();
	if (!Drawer.IsValid())
	{
		return;
	}

	TMap<FName, TSharedRef<ISidebarDrawerContent>> SortedContentDrawers;
	SortedContentDrawers.Reserve(Drawer->ContentSections.Num());

	ButtonBox->ClearChildren();
	ContentBox->ClearChildren();

	if (Drawer->ContentSections.IsEmpty())
	{
		return;
	}

	SortedContentDrawers = Drawer->ContentSections;
	SortedContentDrawers.ValueSort(
		[](const TSharedRef<ISidebarDrawerContent>& InA, const TSharedRef<ISidebarDrawerContent>& InB)
		{
			const int32 SortOrderA = InA->GetSortOrder();
			const int32 SortOrderB = InB->GetSortOrder();

			if (SortOrderA == SortOrderB)
			{
				static const FName General(TEXT("General"));
				static const FName All(TEXT("All"));

				const FName SectionNameA = InA->GetSectionId();
				const FName SectionNameB = InB->GetSectionId();

				// General first, All last, rest alphabetical
				if (SectionNameA.IsEqual(General) || SectionNameB.IsEqual(All))
				{
					return true;
				}
				if (SectionNameA.IsEqual(All) || SectionNameB.IsEqual(General))
				{
					return false;
				}

				return SectionNameA.LexicalLess(SectionNameB);
			}

			return SortOrderA < SortOrderB;
		});

	const TArray<TSharedRef<ISidebarDrawerContent>> OriginalSectionNamesOrder = GetOrderedSections();
	for (const TSharedRef<ISidebarDrawerContent>& Section : OriginalSectionNamesOrder)
	{
		const FName SectionName = Section->GetSectionId();

		ButtonBox->AddSlot()
			[
				SNew(SBox)
				.Padding(FMargin(0))
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.Visibility(this, &SSidebarDrawerContent::GetSectionButtonVisibility, Section->AsWeak())
				[
					SNew(SCheckBox)
					.Style(FAppStyle::Get(), TEXT("DetailsView.SectionButton"))
					.OnCheckStateChanged(this, &SSidebarDrawerContent::OnSectionSelected, SectionName)
					.IsChecked(this, &SSidebarDrawerContent::GetSectionCheckBoxState, SectionName)
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), TEXT("SmallText"))
						.Text(Section->GetSectionDisplayText())
						.Justification(ETextJustify::Center)
					]
				]
			];
	}

	for (const TPair<FName, TSharedRef<ISidebarDrawerContent>>& SectionPair : SortedContentDrawers)
	{
		AddContentSlot(SectionPair.Value);
	}

	if (Drawer->State.SelectedSections.IsEmpty())
	{
		// Find first section that is visible
		FName FoundSectionName;
		for (const TPair<FName, TSharedRef<ISidebarDrawerContent>>& Section : Drawer->ContentSections)
		{
			if (Section.Value->ShouldShowSection())
			{
				FoundSectionName = Section.Value->GetSectionId();
				break;
			}
		}

		if (FoundSectionName != NAME_None)
		{
			Drawer->State.SelectedSections.Add(FoundSectionName);
		}
	}
}

void SSidebarDrawerContent::OnSectionSelected(const ECheckBoxState InCheckBoxState, const FName InSectionName)
{
	const TSharedPtr<FSidebarDrawer> Drawer = OwnerDrawerWeak.Pin();
	if (!Drawer.IsValid())
	{
		return;
	}

	const FModifierKeysState ModifierKeysState = FSlateApplication::Get().GetModifierKeys();
	const bool bIsModifierDown = ModifierKeysState.IsControlDown() || ModifierKeysState.IsShiftDown();

	if (InCheckBoxState == ECheckBoxState::Checked)
	{
		if (bIsModifierDown)
		{
			Drawer->State.SelectedSections.Add(InSectionName);
		}
		else
		{
			Drawer->State.SelectedSections.Reset();
			Drawer->State.SelectedSections.Add(InSectionName);
		}
	}
	else
	{
		if (bIsModifierDown)
		{
			Drawer->State.SelectedSections.Remove(InSectionName);

			// Force to always have a Selected Section, making it unable to de-select the last one
			if (Drawer->State.SelectedSections.IsEmpty())
			{
				Drawer->State.SelectedSections.Add(InSectionName);
			}
		}
		else
		{
			Drawer->State.SelectedSections.Reset();
			Drawer->State.SelectedSections.Add(InSectionName);
		}
	}
}

bool SSidebarDrawerContent::IsSectionSelected(const FName InSectionName) const
{
	const TSharedPtr<FSidebarDrawer> Drawer = OwnerDrawerWeak.Pin();
	return Drawer.IsValid() && Drawer->State.SelectedSections.Contains(InSectionName);
}

bool SSidebarDrawerContent::ShouldShowSection(const TWeakPtr<ISidebarDrawerContent>& InSectionWeak) const
{
	const TSharedPtr<ISidebarDrawerContent> Section = InSectionWeak.Pin();
	return Section.IsValid() && Section->ShouldShowSection();
}

EVisibility SSidebarDrawerContent::GetSectionButtonVisibility(TWeakPtr<ISidebarDrawerContent> InSectionWeak) const
{
	return ShouldShowSection(InSectionWeak) ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
}

EVisibility SSidebarDrawerContent::GetSectionContentVisibility(const FName InSectionName, TWeakPtr<ISidebarDrawerContent> InSectionWeak) const
{
	return IsSectionSelected(InSectionName) && ShouldShowSection(InSectionWeak)
		? EVisibility::SelfHitTestInvisible
		: EVisibility::Collapsed;
}

ECheckBoxState SSidebarDrawerContent::GetSectionCheckBoxState(const FName InSectionName) const
{
	return IsSectionSelected(InSectionName) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

TArray<TSharedRef<ISidebarDrawerContent>> SSidebarDrawerContent::GetOrderedSections() const
{
	const TSharedPtr<FSidebarDrawer> Drawer = OwnerDrawerWeak.Pin();
	if (!Drawer.IsValid())
	{
		return {};
	}

	TArray<TSharedRef<ISidebarDrawerContent>> OrderedSections;

	for (const TPair<FName, TSharedRef<ISidebarDrawerContent>>& SectionPair : Drawer->ContentSections)
	{
		const bool bContains = OrderedSections.ContainsByPredicate(
			[&SectionPair](const TSharedRef<ISidebarDrawerContent>& InOther)
			{
				return SectionPair.Value->GetSectionId() == InOther->GetSectionId();
			});
		if (!bContains)
		{
			OrderedSections.Add(SectionPair.Value);
		}
	}

	return MoveTemp(OrderedSections);
}

void SSidebarDrawerContent::AddContentSlot(const TSharedRef<ISidebarDrawerContent>& InDrawerContent)
{
	SVerticalBox::FScopedWidgetSlotArguments Slot = ContentBox->AddSlot();
	Slot.Padding(0.f, 0.f, 0.f, 2.f)
		[
			SNew(SBox)
			.Visibility(this, &SSidebarDrawerContent::GetSectionContentVisibility
				, InDrawerContent->GetSectionId(), InDrawerContent->AsWeak())
			[
				InDrawerContent->CreateContentWidget()
			]
		];

	const TOptional<float> FillHeight = InDrawerContent->GetSectionFillHeight();
	if (FillHeight.IsSet())
	{
		Slot.FillHeight(FillHeight.GetValue());
	}
	else
	{
		Slot.AutoHeight();
	}
}

#undef LOCTEXT_NAMESPACE 
