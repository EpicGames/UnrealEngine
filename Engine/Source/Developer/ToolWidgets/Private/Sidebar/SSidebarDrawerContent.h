// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class FName;
class FSidebarDrawer;
class ISidebarDrawerContent;
class SVerticalBox;
class SWrapBox;

/** Handles drawer multi-section display */
class SSidebarDrawerContent : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSidebarDrawerContent) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<FSidebarDrawer>& InOwnerDrawerWeak);

	void BuildContent();

private:
	void OnSectionSelected(const ECheckBoxState InCheckBoxState, const FName InSectionName);

	bool IsSectionSelected(const FName InSectionName) const;

	bool ShouldShowSection(const TWeakPtr<ISidebarDrawerContent>& InSectionWeak) const;

	EVisibility GetSectionButtonVisibility(TWeakPtr<ISidebarDrawerContent> InSectionWeak) const;
	
	EVisibility GetSectionContentVisibility(const FName InSectionName, TWeakPtr<ISidebarDrawerContent> InSectionWeak) const;

	ECheckBoxState GetSectionCheckBoxState(const FName InSectionName) const;

	TArray<TSharedRef<ISidebarDrawerContent>> GetOrderedSections() const;

	void AddContentSlot(const TSharedRef<ISidebarDrawerContent>& InDrawerContent);

	TWeakPtr<FSidebarDrawer> OwnerDrawerWeak;

	TSharedPtr<SWrapBox> ButtonBox;

	TSharedPtr<SVerticalBox> ContentBox;
};
