// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "SidebarButtonMenuContext.generated.h"

class FSidebarDrawer;
class SSidebar;

UCLASS()
class USidebarButtonMenuContext : public UObject
{
	GENERATED_BODY()

public:
	void Init(const TWeakPtr<SSidebar>& InSidebarWeak, const TWeakPtr<FSidebarDrawer>& InDrawerWeak)
	{
		SidebarWeak = InSidebarWeak;
		DrawerWeak = InDrawerWeak;
	}

	TSharedPtr<SSidebar> GetSidebarWidget() const
	{
		return SidebarWeak.Pin();
	}

	TSharedPtr<FSidebarDrawer> GetDrawer() const
	{
		return DrawerWeak.Pin();
	}

protected:
	TWeakPtr<SSidebar> SidebarWeak;

	TWeakPtr<FSidebarDrawer> DrawerWeak;
};
