// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Sidebar/SidebarDrawerConfig.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class ISidebarDrawerContent;
class SSidebarDrawer;
class SWidget;

DECLARE_DELEGATE_TwoParams(FOnSidebarDrawerOpenClose, const FName /*InUniqueId*/, const bool /*bOpen*/)

class FSidebarDrawer
{
public:
	FSidebarDrawer(FSidebarDrawerConfig&& InDrawerConfig)
		: Config(MoveTemp(InDrawerConfig))
	{}

	bool operator==(const FName InOtherId) const
	{
		return Config.UniqueId == InOtherId;
	}

	bool operator!=(const FName InOtherId) const
	{
		return Config.UniqueId != InOtherId;
	}

	bool operator==(const FSidebarDrawer& InOther) const
	{
		return Config == InOther.Config;
	}

	bool operator!=(const FSidebarDrawer& InOther) const
	{
		return Config != InOther.Config;
	}

	FName GetUniqueId() const
	{
		return Config.UniqueId;
	}

	FSidebarDrawerConfig Config;

	/** Tab button widget for this drawer. */
	TSharedPtr<SWidget> ButtonWidget;

	/** Tab sliding drawer widget that contains the content. */
	TSharedPtr<SSidebarDrawer> DrawerWidget;

	/** The content widget contains the section widgets. */
	TSharedPtr<SWidget> ContentWidget;

	TMap<FName, TSharedRef<ISidebarDrawerContent>> ContentSections;

	bool bDisablePin = false;
	bool bDisableDock = false;

	bool bIsOpen = false;
	FSidebarDrawerState State;

	FOnSidebarDrawerOpenClose OnDrawerOpenCloseDelegate;
};
