// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sidebar/SidebarState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SidebarState)

bool FSidebarState::IsValid() const
{
	return !(bHidden == false && DrawerSize == 0.f && DrawerStates.IsEmpty());
}

bool FSidebarState::IsHidden() const
{
	return bHidden;
}

bool FSidebarState::IsVisible() const
{
	return !bHidden;
}

void FSidebarState::SetHidden(const bool bInHidden)
{
	bHidden = bInHidden;
}

void FSidebarState::SetVisible(const bool bInVisible)
{
	bHidden = !bInVisible;
}

float FSidebarState::GetDrawerSize() const
{
	return (DrawerSize <= MinSize) ? DefaultSize : DrawerSize;
}

void FSidebarState::SetDrawerSize(const float InSize)
{
	DrawerSize = InSize;
}

void FSidebarState::GetDrawerSizes(float& OutDrawerSize, float& OutContentSize) const
{
	OutDrawerSize = (OutDrawerSize <= MinSize) ? DefaultSize : DrawerSize;
	OutContentSize = (OutContentSize <= MinSize) ? DefaultSize : ContentSize;

	const float TotalSize = OutDrawerSize + OutContentSize;
	if (TotalSize < 1.f)
	{
		OutContentSize = 1.f - OutDrawerSize;
	}
}

void FSidebarState::SetDrawerSizes(const float InDrawerSize, const float InContentSize)
{
	DrawerSize = InDrawerSize;
	ContentSize = InContentSize;
}

const TArray<FSidebarDrawerState>& FSidebarState::GetDrawerStates() const
{
	return DrawerStates;
}

FSidebarDrawerState& FSidebarState::FindOrAddDrawerState(const FSidebarDrawerState& InDrawerState)
{
	for (FSidebarDrawerState& State : DrawerStates)
	{
		if (State.DrawerId == InDrawerState.DrawerId)
		{
			return State;
		}
	}
	return DrawerStates.Add_GetRef(InDrawerState);
}

const FSidebarDrawerState* FSidebarState::FindDrawerState(const FSidebarDrawerState& InDrawerState)
{
	for (FSidebarDrawerState& State : DrawerStates)
	{
		if (State.DrawerId == InDrawerState.DrawerId)
		{
			return &State;
		}
	}
	return nullptr;
}

void FSidebarState::SaveDrawerState(const FSidebarDrawerState& InState)
{
	for (FSidebarDrawerState& State : DrawerStates)
	{
		if (State.DrawerId.IsEqual(InState.DrawerId))
		{
			State = InState;
			return;
		}
	}
	DrawerStates.Add(InState);
}
