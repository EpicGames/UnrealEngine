// Copyright Epic Games, Inc. All Rights Reserved.

#include "Dialog/DialogCommands.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FToolWidgetsModule
	: public IModuleInterface
{
public:

	// IModuleInterface interface
	
	virtual void StartupModule() override
	{
		// Register the dialog commands
		FDialogCommands::Register();
	}
	virtual void ShutdownModule() override { }
};


IMPLEMENT_MODULE(FToolWidgetsModule, ToolWidgets);
