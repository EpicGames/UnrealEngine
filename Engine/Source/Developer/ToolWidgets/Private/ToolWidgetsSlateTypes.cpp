// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolWidgetsSlateTypes.h"

#include "ToolWidgetsStylePrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ToolWidgetsSlateTypes)

namespace UE::ToolWidgets::Private
{
	inline FName LexToName(const EActionButtonType ActionButtonType)
	{
		static const TMap<EActionButtonType, const FName> Lookup = {
			{ EActionButtonType::Default, TEXT("Default") },
			{ EActionButtonType::Simple, TEXT("Simple") },
			{ EActionButtonType::Primary, TEXT("Primary") },
			{ EActionButtonType::Positive, TEXT("Positive") },
			{ EActionButtonType::Warning, TEXT("Warning") },
			{ EActionButtonType::Error, TEXT("Error") },
		};

		if (const FName* FoundName = Lookup.Find(ActionButtonType))
		{
			return *FoundName;
		}

		return TEXT("Invalid");
	}

	inline EActionButtonType NameToEnum(const FName ActionButtonTypeName)
	{
		static const TMap<const FName, EActionButtonType> Lookup = {
			{ TEXT("Default"), EActionButtonType::Default },
			{ TEXT("Simple"), EActionButtonType::Simple },
			{ TEXT("Primary"), EActionButtonType::Primary },
			{ TEXT("Positive"), EActionButtonType::Positive },
			{ TEXT("Warning"), EActionButtonType::Warning },
			{ TEXT("Error"), EActionButtonType::Error },
		};

		if (const EActionButtonType* FoundEnum = Lookup.Find(ActionButtonTypeName))
		{
			return *FoundEnum;
		}

		return EActionButtonType::Default;
	}

}

const FName FActionButtonStyle::TypeName(TEXT("FActionButtonStyle"));

FActionButtonStyle::FActionButtonStyle()
	: bHasDownArrow(false)
	, HorizontalContentAlignment(HAlign_Center)
{
}

void FActionButtonStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
	if (IconBrush.IsSet())
	{
		OutBrushes.Add(&IconBrush.GetValue());
	}

	ButtonStyle.GetResources(OutBrushes);
	ComboButtonStyle.GetResources(OutBrushes);
	TextBlockStyle.GetResources(OutBrushes);
}

const FActionButtonStyle& FActionButtonStyle::GetDefault()
{
	static FActionButtonStyle Default;
	return Default;
}

const FName FActionButtonStyle::GetTypeName() const
{
	return TypeName;
}

EActionButtonType FActionButtonStyle::GetActionButtonType() const
{
	return UE::ToolWidgets::Private::NameToEnum(ActionButtonType);
}

FActionButtonStyle& FActionButtonStyle::SetActionButtonType(const EActionButtonType InActionButtonType)
{
	ActionButtonType = UE::ToolWidgets::Private::LexToName(InActionButtonType);
	return *this;
}

FActionButtonStyle& FActionButtonStyle::SetButtonStyle(const FButtonStyle& InButtonStyle)
{
	ButtonStyle = InButtonStyle;
	return *this;
}

const FButtonStyle& FActionButtonStyle::GetIconButtonStyle() const
{
	return IconButtonStyle.Get(ButtonStyle);
}

FActionButtonStyle& FActionButtonStyle::SetIconButtonStyle(const FButtonStyle& InButtonStyle)
{
	IconButtonStyle = InButtonStyle;
	return *this;
}

FMargin FActionButtonStyle::GetButtonContentPadding() const
{
	static const FMargin DefaultButtonContentPadding = FMargin(
		UE::ToolWidgets::Private::FToolWidgetsStylePrivate::FActionButton::DefaultHorizontalPadding,
		UE::ToolWidgets::Private::FToolWidgetsStylePrivate::FActionButton::DefaultVerticalPadding);

	return ButtonContentPadding.IsSet()
		? ButtonContentPadding.GetValue()
		: DefaultButtonContentPadding;
}

FMargin FActionButtonStyle::GetComboButtonContentPadding() const
{
	return ComboButtonContentPadding.IsSet()
		? ComboButtonContentPadding.GetValue()
		: ComboButtonStyle.ContentPadding;
}

FActionButtonStyle& FActionButtonStyle::SetButtonContentPadding(const FMargin& InContentPadding)
{
	ButtonContentPadding = InContentPadding;
	return *this;
}

FActionButtonStyle& FActionButtonStyle::SetComboButtonStyle(const FComboButtonStyle& InComboButtonStyle)
{
	ComboButtonStyle = InComboButtonStyle;
	return *this;
}

FActionButtonStyle& FActionButtonStyle::SetHasDownArrow(const bool& bInHasDownArrow)
{
	bHasDownArrow = bInHasDownArrow;
	return *this;
}

FActionButtonStyle& FActionButtonStyle::SetComboButtonContentPadding(const FMargin& InContentPadding)
{
	ComboButtonContentPadding = InContentPadding;
	return *this;
}

FActionButtonStyle& FActionButtonStyle::SetHorizontalContentAlignment(const EHorizontalAlignment& InAlignment)
{
	HorizontalContentAlignment = InAlignment;
	return *this;
}

FActionButtonStyle& FActionButtonStyle::SetTextBlockStyle(const FTextBlockStyle& InTextBlockStyle)
{
	TextBlockStyle = InTextBlockStyle;
	return *this;
}

FActionButtonStyle& FActionButtonStyle::SetIconBrush(const FSlateBrush& InIconBrush)
{
	IconBrush = InIconBrush;
	return *this;
}

FActionButtonStyle& FActionButtonStyle::SetIconColorAndOpacity(const FSlateColor& InIconColorAndOpacity)
{
	IconColorAndOpacity = InIconColorAndOpacity;
	return *this;
}

FActionButtonStyle& FActionButtonStyle::SetIconNormalPadding(const FMargin& InIconNormalPadding)
{
	IconNormalPadding = InIconNormalPadding;
	return *this;
}

FActionButtonStyle& FActionButtonStyle::SetIconPressedPadding(const FMargin& InIconPressedPadding)
{
	IconPressedPadding = InIconPressedPadding;
	return *this;
}

void FActionButtonStyle::UnlinkColors()
{
	ButtonStyle.UnlinkColors();
	ComboButtonStyle.UnlinkColors();
	TextBlockStyle.UnlinkColors();
}

#if WITH_EDITOR
const TArray<FName>& UToolSlateWidgetTypesFunctionLibrary::GetActionButtonTypeNames()
{
	static const TArray<FName> ActionButtonTypeNames = {
		TEXT("Default"),
		TEXT("Simple"),
		TEXT("Primary"),
		TEXT("Positive"),
		TEXT("Warning"),
		TEXT("Error"),
	};

	return ActionButtonTypeNames;
}
#endif
