// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolWidgetsStyle.h"

#include "Styling/AppStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StyleColors.h"
#include "ToolWidgetsSlateTypes.h"
#include "ToolWidgetsStylePrivate.h"

namespace UE::ToolWidgets
{
	namespace Private
	{
		/** Compensates for Button Style padding. */
		static FMargin ModifyContentPadding(const FButtonStyle& InButtonStyle, const FMargin& InContentPadding)
		{
			FMargin ContentPadding = InContentPadding;
			ContentPadding.Top = ContentPadding.Bottom = InButtonStyle.NormalPadding.Top;
			return ContentPadding;
		}

		static FMargin ModifyContentPadding(const FComboButtonStyle& InComboButtonStyle, const FMargin& InContentPadding)
		{
			return ModifyContentPadding(InComboButtonStyle.ButtonStyle,	InContentPadding);
		}
	}
	
	FName FToolWidgetsStyle::StyleName("ToolWidgets");

	FToolWidgetsStyle& FToolWidgetsStyle::Get()
	{
		static FToolWidgetsStyle Instance;
		return Instance;
	}

	FToolWidgetsStyle::FToolWidgetsStyle()
		: FSlateStyleSet(StyleName)
	{
		SetParentStyleName(FAppStyle::GetAppStyleSetName());

		SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		// SActionButton
		{
			using namespace Private;

			static const FMargin DefaultButtonContentPadding = FMargin(
				FToolWidgetsStylePrivate::FActionButton::DefaultHorizontalPadding,
				FToolWidgetsStylePrivate::FActionButton::DefaultVerticalPadding);

			const FButtonStyle& DefaultButtonStyle = GetWidgetStyle<FButtonStyle>("Button");
			const FComboButtonStyle& DefaultComboButtonStyle = GetWidgetStyle<FComboButtonStyle>("ComboButton");

			FActionButtonStyle ActionButton = FActionButtonStyle();
			{
				ActionButton
					.SetActionButtonType(EActionButtonType::Default)
					.SetButtonStyle(DefaultButtonStyle)
					.SetButtonContentPadding(ModifyContentPadding(DefaultButtonStyle, DefaultButtonContentPadding))
					.SetComboButtonStyle(DefaultComboButtonStyle)
					.SetComboButtonContentPadding(ModifyContentPadding(DefaultButtonStyle, DefaultButtonContentPadding))
					.SetHorizontalContentAlignment(HAlign_Center)
					.SetTextBlockStyle(GetWidgetStyle<FTextBlockStyle>("SmallButtonText"))
					.SetHasDownArrow(false);

				Set("ActionButton", ActionButton);
			}

			FActionButtonStyle PositiveActionButton = ActionButton;
			{
				PositiveActionButton
					.SetActionButtonType(EActionButtonType::Positive)
					.SetIconBrush(*FAppStyle::Get().GetBrush("Icons.Plus"))
					.SetIconColorAndOpacity(FStyleColors::AccentGreen);

				Set("PositiveActionButton", PositiveActionButton);
			}

			FActionButtonStyle NegativeActionButton = ActionButton;
			{
				FActionButtonStyle NegativeActionButtonWarning = NegativeActionButton;

				NegativeActionButtonWarning
					.SetActionButtonType(EActionButtonType::Warning)
					.SetIconBrush(*FAppStyle::Get().GetBrush("Icons.Warning"));

				Set("NegativeActionButton.Warning", NegativeActionButtonWarning);

				FActionButtonStyle NegativeActionButtonError = NegativeActionButton;

				NegativeActionButtonError
					.SetActionButtonType(EActionButtonType::Error)
					.SetIconBrush(*FAppStyle::Get().GetBrush("Icons.Error"));

				Set("NegativeActionButton.Error", NegativeActionButtonError);
			}

			FActionButtonStyle SimpleComboButton = ActionButton;
			{
				const FComboButtonStyle& ComboButtonStyle = GetWidgetStyle<FComboButtonStyle>("SimpleComboButton");

				SimpleComboButton
					.SetActionButtonType(EActionButtonType::Simple)
					.SetComboButtonStyle(ComboButtonStyle)
					.SetButtonStyle(ComboButtonStyle.ButtonStyle)
					.SetComboButtonContentPadding(ModifyContentPadding(ComboButtonStyle, DefaultButtonContentPadding))
					.SetHorizontalContentAlignment(HAlign_Left)
					.SetTextBlockStyle(GetWidgetStyle<FTextBlockStyle>("SmallButtonText"))
					.SetIconColorAndOpacity(FSlateColor::UseForeground());

				Set("SimpleComboButton", SimpleComboButton);
			}

			FActionButtonStyle SimpleButton = ActionButton;
			{
				const FButtonStyle& ButtonStyle = GetWidgetStyle<FButtonStyle>("SimpleButtonLabelAndIcon");

				SimpleButton
					.SetActionButtonType(EActionButtonType::Simple)
					.SetButtonStyle(ButtonStyle)
					.SetButtonContentPadding(ModifyContentPadding(ButtonStyle, DefaultButtonContentPadding))
					.SetHorizontalContentAlignment(HAlign_Left)
					.SetTextBlockStyle(GetWidgetStyle<FTextBlockStyle>("SmallButtonText"))
					.SetIconColorAndOpacity(FSlateColor::UseForeground());

				Set("SimpleButton", SimpleButton);
			}

			FActionButtonStyle PrimaryButton = ActionButton;
			{
				const FButtonStyle& ButtonStyle = GetWidgetStyle<FButtonStyle>("PrimaryButton");

				PrimaryButton
					.SetActionButtonType(EActionButtonType::Primary)
					.SetButtonStyle(ButtonStyle)
					.SetButtonContentPadding(ModifyContentPadding(ButtonStyle, DefaultButtonContentPadding))
					.SetComboButtonContentPadding(ModifyContentPadding(ButtonStyle, DefaultButtonContentPadding))
					.SetIconNormalPadding(GetMargin("PrimaryButtonLabelAndIconNormalPadding"))
					.SetIconPressedPadding(GetMargin("PrimaryButtonLabelAndIconPressedPadding"))
					.SetTextBlockStyle(GetWidgetStyle<FTextBlockStyle>("PrimaryButtonText"))
					.SetIconColorAndOpacity(FSlateColor::UseForeground());

				Set("PrimaryButton", PrimaryButton);
			}
		}

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	FToolWidgetsStyle::~FToolWidgetsStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
}
