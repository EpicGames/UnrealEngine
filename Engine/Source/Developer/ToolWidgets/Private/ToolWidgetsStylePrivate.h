// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

namespace UE::ToolWidgets::Private
{
	/** A supplementary Style container, accessible to the module only. */
	class FToolWidgetsStylePrivate final
	{
	public:
		struct FButton
		{
			static constexpr float DefaultBorderPadding = 2.0f; // This needs to be accounted for when padding buttons to achieve the desired height, may be overridden by ButtonStyle
			static constexpr float DefaultBorderThickness = 1.0f;
		};

		struct FActionButton
		{
			static constexpr float DefaultButtonHeight = 22.0f; // Design standard
			static constexpr float DefaultIconHeight = 16.0f; // Vertical icon space should always be used to affect button size and ensure it's consistent regardless of icon presence
			static constexpr float DefaultIconLabelSpacing = 3.0f;
			static constexpr float DefaultVerticalPadding = (DefaultButtonHeight - DefaultIconHeight) / 2.0f; // Total Button Height - Icon Height (/2 for top and bottom)
			static constexpr float DefaultHorizontalPadding = 2.0f;
		};
	};
}
