// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolWidgetsUtilitiesPrivate.h"

#include "Styling/StyleColors.h"
#include "ToolWidgetsSlateTypes.h"
#include "ToolWidgetsStylePrivate.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::ToolWidgets::Private
{
	namespace ActionButton
	{
		TSharedRef<SWidget> MakeButtonContent(
			const TAttribute<const FSlateBrush*>& InIcon,
			const TAttribute<FSlateColor>& InIconColorAndOpacity,
			const TAttribute<FText>& InText,
			const FTextBlockStyle* InTextBlockStyle)
		{
			check(InIconColorAndOpacity.IsSet() || InIconColorAndOpacity.IsBound());
			check(InTextBlockStyle);

			const bool bCanEverHaveIcon = InIcon.IsBound() || InIcon.Get(nullptr) != nullptr;

			static constexpr float DefaultIconHeight = FToolWidgetsStylePrivate::FActionButton::DefaultIconHeight;
			static constexpr float IconTextPadding = FToolWidgetsStylePrivate::FActionButton::DefaultIconLabelSpacing;


			TAttribute<FMargin> PaddingAttribute = TAttribute<FMargin>::CreateLambda([InIcon, DefaultPadding = IconTextPadding]()
			{
				if (InIcon.Get(nullptr) != nullptr)
				{
					return FMargin(DefaultPadding, 0, 0, 0);
				}

				return FMargin(0);
			});

			const TSharedRef<SHorizontalBox> ButtonContentContainer = SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0)
				[
					SNew(SWidgetSwitcher)
					.WidgetIndex(bCanEverHaveIcon ? 1 : 0)

					+ SWidgetSwitcher::Slot()
					[
						SNew(SSpacer)
						.Size(FVector2D{ 0, DefaultIconHeight })
					]

					+ SWidgetSwitcher::Slot()
					[
						SNew(SImage)
						.Image(InIcon)
						.ColorAndOpacity(InIconColorAndOpacity)
						.Visibility(bCanEverHaveIcon ? EVisibility::HitTestInvisible : EVisibility::Collapsed)
					]
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.FillWidth(1.0f)
				.Padding(PaddingAttribute)
				[
					SNew(STextBlock)
					.TextStyle(InTextBlockStyle)
					.Text(InText)
					.Visibility_Lambda([InText]()
					{
						return InText.Get(
							FText::GetEmpty()).IsEmpty()
							? EVisibility::Collapsed
							: EVisibility::Visible;
					})
				];

			return ButtonContentContainer;
		}

		TSharedRef<SWidget> MakeButtonContent(
			const FActionButtonStyle* InActionButtonStyle,
			const TAttribute<const FSlateBrush*>& InIcon,
			const TAttribute<FSlateColor>& InIconColorAndOpacity,
			const TAttribute<FText>& InText,
			const FTextBlockStyle* InTextBlockStyle)
		{
			check(InActionButtonStyle);

			// Check for widget level override, then style override, otherwise unset
			const TAttribute<const FSlateBrush*> Icon = InIcon.IsSet()
				? InIcon
				: InActionButtonStyle->IconBrush.IsSet()
				? &InActionButtonStyle->IconBrush.GetValue()
				: nullptr;

			TAttribute<FSlateColor> IconColorAndOpacity = FSlateColor::UseForeground();
			if (ensureMsgf(InIconColorAndOpacity.IsSet() || InActionButtonStyle->IconColorAndOpacity.IsSet(), TEXT("The provided IconColorAndOpacity must either be set directly, or stored in the ActionButtonStyle.")))
			{
				// If provided Attribute not set, get from the ActionButtonStyle
				IconColorAndOpacity = InIconColorAndOpacity.Get(InActionButtonStyle->IconColorAndOpacity.Get(IconColorAndOpacity.Get()));
			}

			const FTextBlockStyle* TextBlockStyle = InTextBlockStyle ? InTextBlockStyle : &InActionButtonStyle->TextBlockStyle;

			return MakeButtonContent(Icon, IconColorAndOpacity, InText, TextBlockStyle);
		}

		TAttribute<FMargin> MakeIconPaddingOverride(const TAttribute<const FSlateBrush*>& Icon, const FMargin& ButtonStylePadding, const TOptional<FMargin>& IconPadding)
		{
			if (IconPadding.IsSet())
			{
				if (Icon.IsBound())
				{
					return TAttribute<FMargin>::CreateLambda([Icon, ButtonStylePadding, IconPadding = IconPadding.GetValue()]()
					{
						if (Icon.Get(nullptr) != nullptr)
						{
							return IconPadding;
						}

						return ButtonStylePadding;
					});
				}
				else if (Icon.Get(nullptr) != nullptr)
				{
					return IconPadding.GetValue();
				}
			}
			return {};
		}
	}
}
