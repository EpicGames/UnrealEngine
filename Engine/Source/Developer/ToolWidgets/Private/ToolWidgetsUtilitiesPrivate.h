// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolWidgetsSlateTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"

namespace UE::ToolWidgets::Private
{
	namespace ActionButton
	{
		TSharedRef<SWidget> MakeButtonContent(
			const TAttribute<const FSlateBrush*>& InIcon,
			const TAttribute<FSlateColor>& InIconColorAndOpacity,
			const TAttribute<FText>& InText,
			const FTextBlockStyle* InTextBlockStyle);

		TSharedRef<SWidget> MakeButtonContent(
			const FActionButtonStyle* InActionButtonStyle,
			const TAttribute<const FSlateBrush*>& InIcon,
			const TAttribute<FSlateColor>& InIconColorAndOpacity,
			const TAttribute<FText>& InText,
			const FTextBlockStyle* InTextBlockStyle);

		TAttribute<FMargin> MakeIconPaddingOverride(const TAttribute<const FSlateBrush*>& Icon, const FMargin& ButtonStylePadding, const TOptional<FMargin>& IconPadding);
	}
}
