// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Framework/SlateDelegates.h"
#include "Templates/SharedPointer.h"

#include "SlateDelegateColumns.generated.h"

/**
 * Column added to a widget row when an external widget manages selection for the widget referenced by the row,
 * such as an owning SListView or STreeView
 */
USTRUCT(meta = (DisplayName = "Widget with externally managed selection"))
struct FExternalWidgetSelectionColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	/** Delegate to execute to check the status of if the widget is selected or not
	 * Only needs to be hooked up if an external widget is managing selection, such
	 * as a SListView or STreeView.
	 *
	 * E.g usage: Enables renaming by clicking the label twice if the widget is editable.
	 * @see SInlineEditableTextBlock::IsSelected
	 */
	TSharedPtr<FIsSelected> IsSelected;
};

/**
 * Column added to a widget row to allow systems to provide a context menu for the widget
 */
USTRUCT(meta = (DisplayName = "Widget with context menu"))
struct FWidgetContextMenuColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	TSharedPtr<FOnContextMenuOpening> OnContextMenuOpening;
};

/**
 * Column added to a widget row for table views to allow systems to specify behavior when a row is scrolled into view
 */
USTRUCT(meta = (DisplayName = "Widget with row scrolled into view delegate"))
struct FWidgetRowScrolledIntoView : public FEditorDataStorageColumn
{
	using FOnItemScrolledIntoView = TSlateDelegates<FTedsRowHandle>::FOnItemScrolledIntoView;

	GENERATED_BODY()
	TSharedPtr<FOnItemScrolledIntoView> OnItemScrolledIntoView;
};

/**
 * Column added to a widget row to allow systems to specify behavior when the widget is double clicked
 */
USTRUCT(meta = (DisplayName = "Widget with item scrolled into view delegate"))
struct FWidgetDoubleClickedColumn : public FEditorDataStorageColumn
{
	using FOnMouseButtonDoubleClick = TSlateDelegates<FTedsRowHandle>::FOnMouseButtonDoubleClick;
	
	GENERATED_BODY()
	TSharedPtr<FOnMouseButtonDoubleClick> OnMouseButtonDoubleClick;
};

/**
 * Column that triggers a signal when the label is edited.
 */
USTRUCT(meta = (DisplayName = "On Label Edit"))
struct FWidgetEnterEditModeColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	TSharedPtr<FSimpleDelegate> OnEnterEditMode;
};