// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Widgets/Views/SHeaderRow.h"

#include "SlateHeaderColumns.generated.h"

/**
 * Column added to a header widget row to control the sizing when the default behavior is incorrect.
 */
USTRUCT(meta = (DisplayName = "Header widget with custom size"))
struct FHeaderWidgetSizeColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	EColumnSizeMode::Type ColumnSizeMode;

	/*
	 * Fill: Column stretches to this fraction of the header row
	 * Fixed: Column is fixed at this width in slate units and cannot be resized
	 * Manual: Column defaults to this width in slate units and can be user-sized
	 * FillSized: Column stretches as Fill but is initialized with this width in slate units
	 */
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	float Width;
};