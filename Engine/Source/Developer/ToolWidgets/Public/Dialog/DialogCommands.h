// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Templates/SharedPointer.h"

#define UE_API TOOLWIDGETS_API

class FUICommandInfo;

UE_DECLARE_TCOMMANDS(class FDialogCommands, UE_API)

/**
 * Class containing commands for dialog actions common to most dialogs
 */
	class FDialogCommands : public TCommands<FDialogCommands>
{
public:
	UE_API FDialogCommands();

	/** Trigger the default cancellation action for the current dialog */
	TSharedPtr<FUICommandInfo> Cancel;

	/** Trigger the default confirmation action for the current dialog */
	TSharedPtr<FUICommandInfo> Confirm;

	/** Registers our commands with the binding system */
	UE_API virtual void RegisterCommands() override;
};

#undef UE_API