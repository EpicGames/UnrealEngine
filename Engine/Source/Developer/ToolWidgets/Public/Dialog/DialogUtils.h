// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"

#define UE_API TOOLWIDGETS_API

enum class EAppMsgCategory : uint8;
struct FSlateBrush;

struct FDialogButtonTexts
{
	static TOOLWIDGETS_API const FDialogButtonTexts& Get();

	/** Common dialog button texts */
	const FText No;
	const FText Yes;
	const FText YesAll;
	const FText NoAll;
	const FText Cancel;
	const FText Ok;
	const FText Retry;
	const FText Continue;

private:
	FDialogButtonTexts(const FDialogButtonTexts&) = delete;
	FDialogButtonTexts& operator=(const FDialogButtonTexts&) = delete;

	FDialogButtonTexts();
};

class FDialogUtils
{
public:
	/** Returns the Slate brush corresponding to the provided message category */
	static UE_API const FSlateBrush* GetMessageCategoryIcon(const EAppMsgCategory MessageCategory);
};

#undef UE_API
