// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Commands/UICommandList.h"
#include "Styling/AppStyle.h"
#include "Widgets/SWindow.h"

#define UE_API TOOLWIDGETS_API

/** Delegate for functions to get the appropriate widget to focus on activation */
DECLARE_DELEGATE_RetVal(TSharedPtr<SWidget>, FGetFocusWidget);

/**
 * This is a custom dialog class, which allows any Slate widget to be used as the contents,
 * with any number of auto-generated buttons that have any text. 
 * It also supports adding a custom icon to the dialog.
 * 
 * Usage:
 * TSharedRef<SCustomDialog> HelloWorldDialog = SNew(SCustomDialog)
		.Title(FText(LOCTEXT("HelloWorldTitleExample", "Hello, World!")))
		.Content()
		[
			SNew(SImage).Image(FName(TEXT("Hello"))))
		]
		.Buttons({
			SCustomDialog::FButton(LOCTEXT("OK", "OK")),
			SCustomDialog::FButton(LOCTEXT("Cancel", "Cancel"))
		});

   // returns 0 when OK is pressed, 1 when Cancel is pressed, -1 if the window is closed
   const int ButtonPressed = HelloWorldDialog->ShowModal();

 * Note: If the content is only text, see SMessageDialog.  
 */
class SCustomDialog : public SWindow
{
public:

	/** Determines whether a button is triggered via the Cancel or Confirm hotkeys.
	 * If more than one button is assigned the same role, the last with that role will be used. */
	enum class EButtonRole
	{
		Custom,
		Cancel,
		Confirm
	};

	struct FButton
	{
		FButton(TAttribute<FText> InButtonText, FSimpleDelegate InOnClicked = FSimpleDelegate(), const EButtonRole InButtonRole = EButtonRole::Custom)
			: ButtonText(MoveTemp(InButtonText))
			, ButtonIsEnabled(true)
			, OnClicked(MoveTemp(InOnClicked))
			, ButtonRole(InButtonRole)
		{}

		/** Set whether this button enabled */
		FButton& SetIsEnabled(TAttribute<bool> InButtonIsEnabled) { ButtonIsEnabled = MoveTemp(InButtonIsEnabled); return *this; }

		/** Set the tooltip text to use for this button */
		FButton& SetToolTipText(TAttribute<FText> InButtonToolTipText) { ButtonToolTipText = MoveTemp(InButtonToolTipText); return *this; }

		/** Primary buttons are highlighted */
		FButton& SetPrimary(bool bValue) { bIsPrimary = bValue; return *this; }

		/** Called when the button is clicked */
		FButton& SetOnClicked(FSimpleDelegate InOnClicked) { OnClicked = MoveTemp(InOnClicked); return *this; }

		/** Button role is used to determine whether the button should be triggered via the Cancel and Confirm hotkeys.
		 * The Cancel and Confirm roles should each only be used on a single button. */
		FButton& SetButtonRole(const EButtonRole InButtonRole) { ButtonRole = InButtonRole; return *this; }

		/**
		 * Whether to focus this button. Focus rules:
		 *  1: If a button has SetFocus(), use the last one
		 *  2: If a button is marked primary, use the last one
		 *  3: If a button has the Confirm role, use the last one
		 *  3: Otherwise, use the last button
		 */
		FButton& SetFocus() { bShouldFocus = true; return *this; }
		
		TAttribute<FText> ButtonText;
		TAttribute<FText> ButtonToolTipText;
		TAttribute<bool> ButtonIsEnabled;
		FSimpleDelegate OnClicked;
		EButtonRole ButtonRole;
		bool bIsPrimary = false;
		bool bShouldFocus = false;
	};

	SLATE_BEGIN_ARGS(SCustomDialog) 
		: _AutoCloseOnButtonPress(true)
		, _Icon(nullptr)
		, _HAlignIcon(HAlign_Left)
		, _VAlignIcon(VAlign_Center)
		, _HAlignButtonBox(HAlign_Fill)
		, _VAlignButtonBox(VAlign_Fill)
		, _RootPadding(FMargin(4.f))
		, _ButtonAreaPadding(FMargin(20.f, 16.f, 0.f, 0.f))
		, _UseScrollBox(true)
		, _ScrollBoxMaxHeight(300.f)
		, _HAlignContent(HAlign_Left)
		, _VAlignContent(VAlign_Center)
		, _ClientSize(FVector2f::ZeroVector)
	{
		_AccessibleParams = FAccessibleWidgetData(EAccessibleBehavior::Auto);
	}

		/*********** Functional ***********/
		
		/** Title to display for the dialog. */
		SLATE_ARGUMENT(FText, Title)

		/** The content to display above the button; icon is optionally created to the left of it.  */
		SLATE_NAMED_SLOT(FArguments, Content)
	
		/** The buttons that this dialog should have. One or more buttons must be added.*/
		SLATE_ARGUMENT(TArray<FButton>, Buttons)

		/** Event triggered when the dialog is closed, either because one of the buttons is pressed, or the windows is closed. */
		SLATE_EVENT(FSimpleDelegate, OnClosed)

		/** Provides default values for SWindow::FArguments not overriden by SCustomDialog. */
		SLATE_ARGUMENT(SWindow::FArguments, WindowArguments)

		/** Whether to automatically close this window when any auto-generated button is pressed (default: true) */
		SLATE_ARGUMENT(bool, AutoCloseOnButtonPress)

		/*********** Cosmetic ***********/

		/** Optional icon to display (default: empty, translucent)*/
		SLATE_ATTRIBUTE(const FSlateBrush*, Icon)
	
		/** When specified, ignore the brushes size and report the DesiredSizeOverride as the desired image size (default: use icon size) */
		SLATE_ATTRIBUTE(TOptional<FVector2D>, IconDesiredSizeOverride)

		/** Alignment of icon (default: HAlign_Left)*/
		SLATE_ARGUMENT(EHorizontalAlignment, HAlignIcon)

		/** Alignment of icon (default: VAlign_Center) */
		SLATE_ARGUMENT(EVerticalAlignment, VAlignIcon)
	
	
		/** Custom widget placed before the auto-generated buttons */
		SLATE_NAMED_SLOT(FArguments, BeforeButtons)

		/** HAlign to use for Button Box slot (default: HAlign_Fill) */
		SLATE_ARGUMENT(EHorizontalAlignment, HAlignButtonBox)

		SLATE_ARGUMENT_DEPRECATED(EVerticalAlignment, VAlignButtonBox, 5.8, "The VAlignButtonBox argument is deprecated and not in use anymore")

		/** Padding to apply to the widget embedded in the window, i.e. to all widgets contained in the window (default: {4,4,4,4} )*/
		SLATE_ATTRIBUTE(FMargin, RootPadding)

		/** Padding to apply around the layout holding the buttons (default: {20,16,0,0}) */
		SLATE_ATTRIBUTE(FMargin, ButtonAreaPadding)

		/** Padding to apply to DialogContent - you can use it to move away from the icon (default: {0,0,0,0}) */
		SLATE_ATTRIBUTE(FMargin, ContentAreaPadding)
	

		/** Should this dialog use a scroll box for over-sized content? (default: true) */
		SLATE_ARGUMENT(bool, UseScrollBox)

		/** Max height for the scroll box (default: 300) */
		SLATE_ARGUMENT(float, ScrollBoxMaxHeight)

	
		/** HAlign to use for Content slot (default: HAlign_Left) */
		SLATE_ARGUMENT(EHorizontalAlignment, HAlignContent)
	
		/** VAlign to use for Content slot (default: VAlign_Center) */
		SLATE_ARGUMENT(EVerticalAlignment, VAlignContent)

		/** What the initial size of the window should be. If left at Zero, the window will be AutoSized. */
		SLATE_ARGUMENT(FVector2D, ClientSize)

		/** Function to call when the Cancel hotkey (Escape) is pressed */
		SLATE_ARGUMENT(FSimpleDelegate, OnCancelHotkeyPressed)

		/** Function to call when the Confirm hotkey (Enter) is pressed */
		SLATE_ARGUMENT(FSimpleDelegate, OnConfirmHotkeyPressed)

		/** Function to call when activating to get the widget to focus, for use when the widget needed is dynamic */
		SLATE_ARGUMENT(FGetFocusWidget, GetWidgetToFocusOnActivate)

		/********** Legacy - do not use **********/
		
		/** Optional icon to display in the dialog (default: none) */
		UE_DEPRECATED(5.1, "Use Icon() instead")
		TOOLWIDGETS_API FArguments& IconBrush(FName InIconBrush);
	
		/** Content for the dialog (deprecated - use Content instead)*/
		SLATE_ARGUMENT_DEPRECATED(TSharedPtr<SWidget>, DialogContent, 5.1, "Use Content() instead.")

	SLATE_END_ARGS()
	
	UE_API void Construct(const FArguments& InArgs);

	/** Show the dialog.
	 * This method will return immediately.
	 */ 
	UE_API void Show();

	/** Show a modal dialog. Will block until an input is received.
	 * Returns the index of the button that was pressed or InDefaultReturnValue if in unattended mode.
	 */
	UE_API int32 ShowModal(const int32 InDefaultReturnValue = INDEX_NONE);

	/** Assigns a function to the Cancel hotkey */
	UE_API void SetOnCancelHotkeyPressed(const FSimpleDelegate& InOnCancelHotkeyPressed);

	/** Assigns a function to the Confirm hotkey */
	UE_API void SetOnConfirmHotkeyPressed(const FSimpleDelegate& InOnConfirmHotkeyPressed);

	/** Sets the function to call when activating to get the widget to focus */
	UE_API void SetGetWidgetToFocusOnActivate(const FGetFocusWidget& InGetWidgetToFocusOnActivate);

	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	UE_API virtual bool OnIsActiveChanged(const FWindowActivateEvent& ActivateEvent) override;

private:

	TSharedRef<SWidget> CreateContentBox(const FArguments& InArgs);
	TSharedRef<SWidget> CreateButtonBox(const FArguments& InArgs);
	
	FReply OnButtonClicked(FSimpleDelegate OnClicked, int32 ButtonIndex);
	void OnHotkeyPressed(FSimpleDelegate OnClicked, int32 ButtonIndex);

	bool CanExecuteCancel() const;
	void ExecuteCancel();

	bool CanExecuteConfirm() const;
	void ExecuteConfirm();
	
	/** The index of the button that was pressed last. */
	int32 LastPressedButton = INDEX_NONE;

	FSimpleDelegate OnClosed;

	// Whether to close the window when any button is pressed or triggered via hotkey
	bool bAutoCloseOnButtonPress = true;

	// Whether a Cancel button has been generated
	bool bGeneratedCancelButton = false;

	// Whether a Confirm button has been generated
	bool bGeneratedConfirmButton = false;

	// Function to call when the Cancel hotkey (Escape) is pressed 
	FSimpleDelegate OnCancelHotkeyPressed;

	// Function to call when the Confirm hotkey (Enter) is pressed 
	FSimpleDelegate OnConfirmHotkeyPressed;

	/** Function to call when activating to get the widget to focus */
	FGetFocusWidget GetWidgetToFocusOnActivate;

	/** Command list for the dialog */
	TSharedPtr<FUICommandList> CommandList;
};

#undef UE_API
