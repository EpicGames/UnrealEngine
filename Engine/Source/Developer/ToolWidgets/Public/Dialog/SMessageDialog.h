// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SCustomDialog.h"

#define UE_API TOOLWIDGETS_API

/**
 * Special case of SCustomDialog dedicated to only displaying text messages.
 * This class enforces uniform style and also adds a button for copying the message.
 */
class SMessageDialog : public SCustomDialog
{
public:
	// Convenience for code using this class
	using FButton = SCustomDialog::FButton;

	SLATE_BEGIN_ARGS(SMessageDialog)
		: _AutoCloseOnButtonPress(true)
		, _UseRichText(true)
		, _DecoratorStyleSet(nullptr)
		, _Icon(nullptr)
		, _UseScrollBox(true)
		, _ScrollBoxMaxHeight(300.f)
		, _WrapMessageAt(512.f)
	{}

		/********** Functional **********/
	
		/** Title to display for the dialog. */
		SLATE_ARGUMENT(FText, Title)
	
		/** Message content */
		SLATE_ARGUMENT(FText, Message)
	
		/** The buttons that this dialog should have. One or more buttons must be added.*/
		SLATE_ARGUMENT(TArray<FButton>, Buttons)

		/** Event triggered when the dialog is closed, either because one of the buttons is pressed, or the windows is closed. */
		SLATE_EVENT(FSimpleDelegate, OnClosed)

		/** Provides default values for SWindow::FArguments not overriden by SCustomDialog. */
		SLATE_ARGUMENT(SWindow::FArguments, WindowArguments)
	
		/** Whether to automatically close this window when any button is pressed (default: true) */
		SLATE_ARGUMENT(bool, AutoCloseOnButtonPress)
	
		/** Whether to use rich-text (true) or plain-text (false) (default: true) */
		SLATE_ARGUMENT(bool, UseRichText)

		/** Text decorators used while parsing the rich text messages (requires UseRichText: true) */
		SLATE_ARGUMENT(TArray<TSharedRef<class ITextDecorator>>, Decorators)

		/** Style set used to look up styles used by decorators for rich text messages (requires UseRichText: true) */
		SLATE_ARGUMENT(const ISlateStyle*, DecoratorStyleSet)
		
		/********** Cosmetic **********/
	
		/** Optional icon to display in the dialog. (default: empty) */
		SLATE_ARGUMENT(const FSlateBrush*, Icon)

		/** Should this dialog use a scroll box for over-sized content? (default: true) */
		SLATE_ARGUMENT(bool, UseScrollBox)

		/** Max height for the scroll box (default: 300) */
		SLATE_ARGUMENT(float, ScrollBoxMaxHeight)

		/** When to wrap the message text (default: 512) */
		SLATE_ATTRIBUTE(float, WrapMessageAt)
	
		/** Minimum width for the text part of the message box. Optional for very short message text to add breathing space to the layout. */
		SLATE_ATTRIBUTE(float, ContentMinWidth)

	SLATE_END_ARGS()
	
	UE_API void Construct(const FArguments& InArgs);

	UE_API virtual	FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	UE_API virtual	FReply OnCopyMessage();

private:

	FText Message;
	
	void CopyMessageToClipboard();
};

#undef UE_API
