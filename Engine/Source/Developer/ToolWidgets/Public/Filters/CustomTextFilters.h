// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/FilterBase.h"
#include "Misc/TextFilter.h"

#include "CustomTextFilters.generated.h"

#define LOCTEXT_NAMESPACE "CustomTextFilters"

/* Struct containing the data that SCustomTextFilterDialog is currently editing */
USTRUCT()
struct FCustomTextFilterData
{
	GENERATED_BODY()

	UPROPERTY()
	FText FilterLabel;
	
	UPROPERTY()
	FText FilterString;

	UPROPERTY()
	FLinearColor FilterColor;

	FCustomTextFilterData()
		: FilterColor(FLinearColor::White)
	{
		
	}
};

/** Interface class to define how a custom text filter converts to/from FCustomTextFilterData
 *  FCustomTextFilter contains a generic implementation that can be used for most cases, look at
 *  FFrontendFilter_CustomText in the Content Browser for an example of a specific implementation
 */
template<typename FilterType>
class ICustomTextFilter
{
public:

	virtual ~ICustomTextFilter() = default;
	
	/** Set the internals of this filter from an FCustomTextFilterData */
	virtual void SetFromCustomTextFilterData(const FCustomTextFilterData& InFilterData) = 0;

	/** Create an FCustomTextFilterData from the internals of this filter */
	virtual FCustomTextFilterData CreateCustomTextFilterData() const = 0;

	/** Get the actual filter */
	virtual TSharedPtr<FFilterBase<FilterType>> GetFilter() = 0;
};


/* A generic CustomTextFilter that can be created by providing a TTextFilter.
 * Provide a delegate to a filter bar widget's CreateTextFilter argument that creates an instance of this class
 * to handle most generic text comparisons.
 */
template<typename FilterType>
class FCustomTextFilter :
	public FFilterBase<FilterType>,
	public ICustomTextFilter<FilterType>,
	public TSharedFromThis<FCustomTextFilter<FilterType>>
{
public:

	FCustomTextFilter(TSharedPtr<TTextFilter<FilterType>> InTextFilter)
	: FFilterBase<FilterType>(nullptr)
	, TextFilter(InTextFilter)
	{
		
	}

	/** Returns the system name for this filter */
	virtual FString GetName() const override
	{
		return GetFilterTypeName().ToString();
	}

	/** Returns the human readable name for this filter */
	virtual FText GetDisplayName() const override
	{
		return DisplayName; 
	}

	/** Set the human readable name for this filter */
	virtual void SetDisplayName(const FText& InDisplayName)
	{
		DisplayName = InDisplayName;
	}

	/** Returns the tooltip for this filter, shown in the filters menu */
	virtual FText GetToolTipText() const override
	{
		return FText::Format(LOCTEXT("CustomTextFiltersTooltip", "Filter by string \"{0}\""), TextFilter->GetRawFilterText()); // The tooltip will display the filter string
	}

	/** Returns the color this filter button will be when displayed as a button */
	virtual FLinearColor GetColor() const override
	{
		return Color;
	}

	/** Set the color this filter button will be when displayed as a button */
	virtual void SetColor(const FLinearColor& InColor)
	{
		Color = InColor;
	}

	/** Returns the name of the icon to use in menu entries */
	virtual FName GetIconName() const override
	{
		return NAME_None;
	}

	/** Returns true if the filter should be in the list when disabled and not in the list when enabled */
	virtual bool IsInverseFilter() const override
	{
		return false;
	}

	virtual bool PassesFilter( FilterType InItem ) const override
	{
		return TextFilter->PassesFilter(InItem);
	}

	/** Get the actual text this filter is using to test against */
	virtual FText GetFilterString() const
	{
		return TextFilter->GetRawFilterText();
	}

	/** Set the actual text this filter is using to test against */
	virtual void SetFilterString(const FText& InFilterString)
	{
		TextFilter->SetRawFilterText(InFilterString);
	}

	/** All FCustomTextFilters have the same internal name, this is a helper function to get that name to test against */
	static FName GetFilterTypeName()
	{
		return FName("CustomTextFilter");
	}

	// ICustomTextFilter interface
	
	/** Set the internals of this filter from an FCustomTextFilterData */
	virtual void SetFromCustomTextFilterData(const FCustomTextFilterData& InFilterData) override
	{
		SetDisplayName(InFilterData.FilterLabel);
		SetColor(InFilterData.FilterColor);
		SetFilterString(InFilterData.FilterString);
	}

	/** Create an FCustomTextFilterData from the internals of this filter */
	virtual FCustomTextFilterData CreateCustomTextFilterData() const override
	{
		FCustomTextFilterData CustomTextFilterData;
		CustomTextFilterData.FilterLabel = GetDisplayName();
		CustomTextFilterData.FilterColor = GetColor();
		CustomTextFilterData.FilterString = GetFilterString();

		return CustomTextFilterData;
	}
	
	/** Get the actual filter */
	virtual TSharedPtr<FFilterBase<FilterType>> GetFilter() override
	{
		return this->AsShared();
	}
	
	// Functionality that is not needed by this filter
	
	/** Notification that the filter became active or inactive */
	virtual void ActiveStateChanged(bool bActive) override
	{
		
	}

	/** Called when the right-click context menu is being built for this filter */
	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override
	{
		
	}
	
	/** Can be overriden for custom FilterBar subclasses to save settings, currently not implemented in any generic Filter Bar */
	virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const override
	{
		
	}

	/** Can be overriden for custom FilterBar subclasses to load settings, currently not implemented in any generic Filter Bar */
	virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) override
	{
		
	}
	
protected:

	/* The actual Text Filter containing information about the text being tested against */
	TSharedPtr<TTextFilter<FilterType>> TextFilter;

	/* The Display Name of this custom filter that the user sees */
	FText DisplayName;

	/* The Color of this filter pill */
	FLinearColor Color;
};

#undef LOCTEXT_NAMESPACE