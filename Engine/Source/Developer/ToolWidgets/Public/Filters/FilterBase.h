// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/FilterCollection.h"
#include "Misc/IFilter.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "UObject/NameTypes.h"

class FMenuBuilder;

class FFilterCategory
{
public:
	FFilterCategory(const FText& InTitle, const FText& InTooltip) : Title(InTitle), Tooltip(InTooltip) {}

	/** The title of this category, used for the menu heading */
	const FText Title;

	/** The menu tooltip for this category */
	const FText Tooltip;
};

template<typename FilterType>
struct FFrontendFilterExternalActivationHelper;

/* The base class for all Filters that can be used with FilterBar Widgets
 * @see FGenericFilter for a subclass that allows easy creation of filters
 */
template<typename FilterType>
class FFilterBase : public IFilter<FilterType>
{
public:

	FFilterBase(TSharedPtr<FFilterCategory> InCategory) : FilterCategory(MoveTemp(InCategory)) {}

	/** Returns the system name for this filter */
	virtual FString GetName() const override = 0;

	/** Returns the human readable name for this filter */
	virtual FText GetDisplayName() const = 0;

	/** Returns the tooltip for this filter, shown in the filters menu */
	virtual FText GetToolTipText() const = 0;

	/** Returns the color this filter button will be when displayed as a button */
	virtual FLinearColor GetColor() const = 0;

	/** Returns the name of the icon to use in menu entries */
	virtual FName GetIconName() const = 0;

	/** If true, the filter will be active in the FilterBar when it is inactive in the UI (i.e the filter pill is grayed out)
	 * @See: FFrontendFilter_ShowOtherDevelopers in Content Browser
	 */
	virtual bool IsInverseFilter() const = 0;

	/** Notification that the filter became active or inactive */
	virtual void ActiveStateChanged(bool bActive) = 0;

	/** Called when the right-click context menu is being built for this filter */
	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) = 0;
	
	/** Can be overriden for custom FilterBar subclasses to save settings, currently not implemented in any gneeric Filter Bar */
	virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const = 0;

	/** Can be overriden for custom FilterBar subclasses to load settings, currently not implemented in any gneeric Filter Bar */
	virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) = 0;

	/** 
	 * Set this filter as checked, and activates or deactivates it.
	 * @note this only works for filters registered by a widget such as SBasicFilterBar that manages them and 
	 *  should be called by external code rather than by the managing widget itself.
	 */
	void SetActive(bool bActive) { SetActiveEvent.Broadcast(bActive); }

	/** Checks whether the filter is checked/pinned and active/enabled */
	bool IsActive() const
	{
		if(IsActiveEvent.IsBound())
		{
			return IsActiveEvent.Execute();
		}

		return false;
	}

	/** 
	 * Manage the state of a filter in a collection taking into account whether it is an inverse filter or not.
	 * Adds or removes the filter from the collcetion and calls ActiveStateChanged
	 */
	static void SetActiveInCollection(TSharedRef<FFilterBase<FilterType>> This, bool bActive, TFilterCollection<FilterType>& Collection)
	{
		if(This->IsInverseFilter())
		{
			//Inverse filters are active when they are "disabled"
			bActive = !bActive;
		}
		This->ActiveStateChanged(bActive);

		if (bActive)
		{
			Collection.Add(This);
		}
		else
		{
			Collection.Remove(This);
		}
	}

	// IFilter implementation
	DECLARE_DERIVED_EVENT( FFilterBase, IFilter<FilterType>::FChangedEvent, FChangedEvent );
	virtual FChangedEvent& OnChanged() override { return ChangedEvent; }

	TSharedPtr<FFilterCategory> GetCategory() { return FilterCategory; }

protected:
	void BroadcastChangedEvent() const { ChangedEvent.Broadcast(); }

	FChangedEvent ChangedEvent;

	/** This event is broadcast to when this filter is set Active */
	DECLARE_EVENT_OneParam(FFilterBase, FSetActiveEvent, bool);
	FSetActiveEvent SetActiveEvent;

	DECLARE_DELEGATE_RetVal(bool, FIsActiveEvent);
	FIsActiveEvent IsActiveEvent;
	
	TSharedPtr<FFilterCategory> FilterCategory;

	friend struct FFrontendFilterExternalActivationHelper<FilterType>;
};
