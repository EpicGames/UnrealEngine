// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/FilterBase.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "UObject/NameTypes.h"


/* A generic filter that can be used with the various FilterBar Widgets
 * Takes in a Category, Name, Display Name and Delegate which specifies how to filter an item
 * You can optionally specify a Tooltip, Icon Name or Color
 */
template<typename FilterType>
class FGenericFilter : public FFilterBase<FilterType>
{
public:

	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnItemFiltered, FilterType);
	
	FGenericFilter(TSharedPtr<FFilterCategory> InCategory, const FString& InName, const FText &DisplayName, FOnItemFiltered InFilterDelegate) :
		FFilterBase<FilterType>(InCategory), Name(InName), DisplayName(DisplayName), FilterDelegate(InFilterDelegate), Color(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f))
	{
		
	}

	/** Returns the system name for this filter */
	virtual FString GetName() const override
	{
		return Name;
	}

	/** Returns the human readable name for this filter */
	virtual FText GetDisplayName() const override
	{
		return DisplayName;
	}

	void SetToolTipText(const FText& InToolTip)
	{
		ToolTip = InToolTip;
	}

	/** Returns the tooltip for this filter, shown in the filters menu */
	virtual FText GetToolTipText() const override
	{
		return ToolTip;
	}

	void SetColor(const FLinearColor& InColor)
	{
		Color = InColor;
	}

	/** Returns the color this filter button will be when displayed as a button */
	virtual FLinearColor GetColor() const override
	{
		return Color;
	}

	void SetIconName(const FName& InIconName)
	{
		IconName = InIconName;
	}

	/** Returns the name of the icon to use in menu entries */
	virtual FName GetIconName() const override
	{
		return IconName;
	}

	/** Returns true if the filter should be in the list when disabled and not in the list when enabled */
	virtual bool IsInverseFilter() const override
	{
		return false; // Generic Filters are meant to be simple
	}

	virtual bool PassesFilter( FilterType InItem ) const override
	{
		if(FilterDelegate.IsBound())
		{
			return FilterDelegate.Execute(InItem);
		}

		return false;
	}

	// Functionality that is not needed by generic filters or currently not implemented
	
	/** Notification that the filter became active or inactive */
	virtual void ActiveStateChanged(bool bActive) override
	{
		
	}

	/** Called when the right-click context menu is being built for this filter */
	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override
	{
		
	}
	
	/** Can be overriden for custom FilterBar subclasses to save settings, currently not implemented in any generic Filter Bar */
	virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const override
	{
		
	}

	/** Can be overriden for custom FilterBar subclasses to load settings, currently not implemented in any generic Filter Bar */
	virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) override
	{
		
	}
	
protected:

	// Required members
	FString Name;
	FText DisplayName;
	FOnItemFiltered FilterDelegate;

	// Optional members
	FText ToolTip;
	FLinearColor Color;
	FName IconName;
	
};

