// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Filters/FilterBase.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Layout/SBorder.h"
#include "ToolMenus.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/SBoxPanel.h"
#include "Misc/FilterCollection.h"
#include "UObject/Object.h"
#include "Misc/TextFilter.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Filters/SCustomTextFilterDialog.h"
#include "SSimpleButton.h"
#include "Styling/StyleColors.h"
#include "Filters/SFilterSearchBox.h"
#include "Framework/Application/SlateApplication.h"

#include "SBasicFilterBar.generated.h"

#define LOCTEXT_NAMESPACE "FilterBar"

/* Delegate used by SBasicFilterBar to populate the add filter menu */
DECLARE_DELEGATE_OneParam(FOnPopulateAddFilterMenu, UToolMenu*)
/** Delegate to extend a menu after it has been generated dynamically. */
DECLARE_DELEGATE_OneParam( FOnExtendAddFilterMenu, UToolMenu* );

/** ToolMenuContext that is used to create the Add Filter Menu */
UCLASS(MinimalAPI)
class UFilterBarContext : public UObject
{
	GENERATED_BODY()

public:
	FOnPopulateAddFilterMenu PopulateFilterMenu;
	FOnExtendAddFilterMenu OnExtendAddFilterMenu;
};

/** Describes if the filters are laid out horizontally (ScrollBox) or vertically (WrapBox) */
UENUM()
enum class EFilterBarLayout : uint8
{
	Horizontal,
	Vertical
};

/** Describes how each individual filter pill looks like */
UENUM()
enum class EFilterPillStyle : uint8
{
	Default, // The default style you see in the Content Browser, pills showing the color and filter name
	Basic    // Simple filter pills that only show the name like you see in the Outliner
};

/** Forward declaration for a helper struct used to activate external filters */
template<typename FilterType>
struct FFrontendFilterExternalActivationHelper;

/**
* A Basic Filter Bar widget, which can be used to filter items of type [FilterType] given a list of custom filters
* @see SFilterBar in EditorWidgets if you want a Filter Bar that includes Asset Type Filters
* NOTE: The filter functions create copies, so you want to use a reference or pointer as the template type when possible
* Sample Usage:
*		SAssignNew(MyFilterBar, SBasicFilterBar<FText&>)
*		.OnFilterChanged() // A delegate for when the list of filters changes
*		.CustomFilters() // An array of filters available to this FilterBar (@see FGenericFilter to create simple delegate based filters)
*
* Use the GetAllActiveFilters() function to get the FilterCollection of Active Filters in this FilterBar, that can be used to filter your items
* Use MakeAddFilterButton() to make the button that summons the dropdown showing all the filters
*/

template<typename FilterType>
class SBasicFilterBar : public SCompoundWidget
{
public:
	/** Delegate for when layout changes */
	DECLARE_DELEGATE_OneParam(FOnFilterBarLayoutChanging, EFilterBarLayout /* NewLayout */)

	/** Delegate for when filters have changed */
	DECLARE_DELEGATE( FOnFilterChanged );
	
	/** Delegate to create a TTextFilter used to compare FilterType with text queries */
    DECLARE_DELEGATE_RetVal( TSharedPtr<ICustomTextFilter<FilterType>>, FCreateTextFilter);
	
 	SLATE_BEGIN_ARGS( SBasicFilterBar<FilterType> )
		: _FilterBarLayout(EFilterBarLayout::Horizontal)
		, _CanChangeOrientation(false)
		, _FilterPillStyle(EFilterPillStyle::Default)
		, _UseSectionsForCategories(false)
		, _bPinAllFrontendFilters(false)
		, _bSortFilters(true)
	{}

 		/** Delegate for when filters have changed */
 		SLATE_EVENT( FOnFilterChanged, OnFilterChanged )

		/** Delegate to extend the Add Filter dropdown */
		SLATE_EVENT( FOnExtendAddFilterMenu, OnExtendAddFilterMenu )

		/** Initial List of Custom Filters that will be added to the AddFilter Menu */
		SLATE_ARGUMENT( TArray<TSharedRef<FFilterBase<FilterType>>>, CustomFilters)

		/** A delegate to create a Text Filter for FilterType items. If provided, will allow creation of custom
		 * text filters from the filter dropdown menu.
		 * @see FCustomTextFilter
		 */
		SLATE_ARGUMENT(FCreateTextFilter, CreateTextFilter)
		
		/** An SFilterSearchBox that can be attached to this filter bar. When provided along with a CreateTextFilter
		 *  delegate, allows the user to save searches from the Search Box as text filters for the filter bar.
		 *	NOTE: Will bind a delegate to SFilterSearchBox::OnSaveSearchClicked
		 */
        SLATE_ARGUMENT(TSharedPtr<SFilterSearchBox>, FilterSearchBox)

		/** Called when the filter bar layout is changing, before the filters are added to the layout */
		SLATE_EVENT(FOnFilterBarLayoutChanging, OnFilterBarLayoutChanging)

		/** The layout that determines how the filters are laid out */
		SLATE_ARGUMENT(EFilterBarLayout, FilterBarLayout)
	
		/** If true, allow dynamically changing the orientation and saving in the config */
		SLATE_ARGUMENT(bool, CanChangeOrientation)

		/** Determines how each individual filter pill looks like */
		SLATE_ARGUMENT(EFilterPillStyle, FilterPillStyle)

		/** Whether to use submenus or sections for categories in the filter menu */
		SLATE_ARGUMENT(bool, UseSectionsForCategories)

		/** Whether we want to automatically pin all frontend filters to make them visible by default. */
		SLATE_ARGUMENT(bool, bPinAllFrontendFilters)

		/** Whether filters are sorted in the bar or appear in the order they are added */
		SLATE_ARGUMENT(bool, bSortFilters)
	
 	SLATE_END_ARGS()

 	void Construct( const FArguments& InArgs )
 	{
		OnFilterChanged = InArgs._OnFilterChanged;
 		OnExtendAddFilterMenu = InArgs._OnExtendAddFilterMenu;
 		CreateTextFilter = InArgs._CreateTextFilter;
 		OnFilterBarLayoutChanging = InArgs._OnFilterBarLayoutChanging;
 		FilterBarLayout = InArgs._FilterBarLayout;
 		bCanChangeOrientation = InArgs._CanChangeOrientation;
 		FilterPillStyle = InArgs._FilterPillStyle;
        bUseSectionsForCategories = InArgs._UseSectionsForCategories;
        bPinAllFrontendFilters = InArgs._bPinAllFrontendFilters;
        bSortFiltersAlphabetically = InArgs._bSortFilters;

 		// We use a widgetswitcher to allow dynamically swapping between the layouts
 		ChildSlot
		[
			SAssignNew(FilterBox, SWidgetSwitcher)
			+ SWidgetSwitcher::Slot()
			[
				SAssignNew(HorizontalFilterBox, SWrapBox)
				.UseAllottedSize(true)
			]
			+ SWidgetSwitcher::Slot()
			[
				SAssignNew(VerticalFilterBox, SScrollBox)
				.Visibility_Lambda([this]()
				{
					return HasAnyFilters() ? EVisibility::Visible : EVisibility::Collapsed;
				})
			]
		];
 		
 		if(FilterBarLayout == EFilterBarLayout::Horizontal)
 		{
 			FilterBox->SetActiveWidget(HorizontalFilterBox.ToSharedRef());
 		}
 		else
 		{
 			FilterBox->SetActiveWidget(VerticalFilterBox.ToSharedRef());
 		}

 		AttachFilterSearchBox(InArgs._FilterSearchBox);

 		// A subclass could be using an external filter collection (SFilterList in ContentBrowser)
 		if(!ActiveFilters)
 		{
 			ActiveFilters = MakeShareable(new TFilterCollection<FilterType>());
 		}

 		// Add the custom filters
 		for (TSharedRef<FFilterBase<FilterType>> Filter : InArgs._CustomFilters)
 		{
 			AddFilter(Filter);
 		}
 	}

	/** Add a custom filter to the FilterBar.
	 * NOTE: This only adds it to the Add Filter Menu, and does not automatically add it to the filter bar or activate it
	 */
	void AddFilter(TSharedRef<FFilterBase<FilterType>> InFilter)
 	{
		AllFrontendFilters.Add(InFilter);
 		
 		// Add the category for the filter
 		if (TSharedPtr<FFilterCategory> Category = InFilter->GetCategory())
 		{
 			AllFilterCategories.AddUnique(Category);
 		}
 			
 		// Bind external activation event
 		FFrontendFilterExternalActivationHelper<FilterType>::BindToFilter(SharedThis(this), InFilter);

 		// Auto add if it is an inverse filters
 		SetFrontendFilterActive(InFilter, false);

 		if(bPinAllFrontendFilters)
 		{
 			AddFilterToBar(InFilter);
 		}
 	}

	EFilterBarLayout GetFilterLayout()
 	{
 		return FilterBarLayout;
 	}
	
	virtual void SetFilterLayout(EFilterBarLayout InFilterBarLayout)
 	{
 		if(!bCanChangeOrientation)
 		{
 			return;
 		}
 		
 		FilterBarLayout = InFilterBarLayout;

 		HorizontalFilterBox->ClearChildren();
 		VerticalFilterBox->ClearChildren();
 		
 		if(FilterBarLayout == EFilterBarLayout::Horizontal)
 		{
 			FilterBox->SetActiveWidget(HorizontalFilterBox.ToSharedRef());
 		}
        else
        {
        	FilterBox->SetActiveWidget(VerticalFilterBox.ToSharedRef());
        }

 		OnFilterBarLayoutChanging.ExecuteIfBound(FilterBarLayout);

 		// Filters needs to be re-added in the new active widget container
 		RequestFiltersUpdate();
 	}

	/** Makes the button that summons the Add Filter dropdown on click */
	static TSharedRef<SWidget> MakeAddFilterButton(TSharedRef<SBasicFilterBar<FilterType>> InFilterBar)
 	{
 		TSharedPtr<SLayeredImage> FilterImage = SNew(SLayeredImage)
		 .Image(FAppStyle::Get().GetBrush("Icons.Filter"))
		 .ColorAndOpacity(FSlateColor::UseForeground());

 		// Badge the filter icon if there are filters active
 		FilterImage->AddLayer(TAttribute<const FSlateBrush*>(InFilterBar, &SBasicFilterBar<FilterType>::GetFilterBadgeIcon));
 		
 		return SNew(SComboButton)
				.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButtonWithIcon"))
				.ForegroundColor(FSlateColor::UseStyle())
				.ToolTipText(LOCTEXT("AddFilterToolTip", "Open the Add Filter Menu to add or manage filters."))
				.OnGetMenuContent(InFilterBar, &SBasicFilterBar<FilterType>::MakeAddFilterMenu)
				.ContentPadding(FMargin(1, 0))
 				// If the filter bar is embedded in a parent menu, this allows the Add Filter Menu to close when clicking on an empty area in the parent menu
 				.CollapseMenuOnParentFocus(true)
				.ButtonContent()
				[
					FilterImage.ToSharedRef()
				];
 	}

	/** Attach an SFilterSearchBox to this filter bar, overriding the SFilterSearchBox:: OnSaveSearchClicked
	 *  event to save a search as a filter
	 */
	void AttachFilterSearchBox(TSharedPtr<SFilterSearchBox> InFilterSearchBox)
 	{
		if(InFilterSearchBox)
		{
			InFilterSearchBox->SetOnSaveSearchHandler(
				 SFilterSearchBox::FOnSaveSearchClicked::CreateSP(this, &SBasicFilterBar<FilterType>::CreateCustomTextFilterFromSearch));
		}
 	}

private:

	/** A class for check boxes in the filter list. If you double click a filter checkbox, you will enable it and disable all others */
	class SFilterCheckBox : public SCheckBox
	{
	public:
		
		void SetOnFilterCtrlClicked(const FOnClicked& NewFilterCtrlClicked)
		{
			OnFilterCtrlClicked = NewFilterCtrlClicked;
		}

		void SetOnFilterAltClicked(const FOnClicked& NewFilterAltClicked)
		{
			OnFilterAltClicked = NewFilterAltClicked;
		}

		void SetOnFilterDoubleClicked( const FOnClicked& NewFilterDoubleClicked )
		{
			OnFilterDoubleClicked = NewFilterDoubleClicked;
		}

		void SetOnFilterMiddleButtonClicked( const FOnClicked& NewFilterMiddleButtonClicked )
		{
			OnFilterMiddleButtonClicked = NewFilterMiddleButtonClicked;
		}

		virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override
		{
			if ( InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && OnFilterDoubleClicked.IsBound() )
			{
				return OnFilterDoubleClicked.Execute();
			}
			else
			{
				return SCheckBox::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
			}
		}

		virtual FReply OnMouseButtonUp( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override
		{
			FReply EventReply = FReply::Handled();
			if (InMouseEvent.IsControlDown() && OnFilterCtrlClicked.IsBound())
			{
				EventReply = OnFilterCtrlClicked.Execute();
			}
			else if (InMouseEvent.IsAltDown() && OnFilterAltClicked.IsBound())
			{
				EventReply = OnFilterAltClicked.Execute();
			}
			else if( InMouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton && OnFilterMiddleButtonClicked.IsBound() )
			{
				EventReply = OnFilterMiddleButtonClicked.Execute();
			}
			else
			{
				SCheckBox::OnMouseButtonUp(InMyGeometry, InMouseEvent);
			}

			bIsPressed = false;
			return EventReply.ReleaseMouseCapture();
		}

	private:
		FOnClicked OnFilterCtrlClicked;
		FOnClicked OnFilterAltClicked;
		FOnClicked OnFilterDoubleClicked;
		FOnClicked OnFilterMiddleButtonClicked;
	};

protected:

	/**
	 * A single filter in the filter list. Can be removed by clicking the remove button on it.
	 */
	class SFilter : public SCompoundWidget
	{
	public:
		DECLARE_DELEGATE_OneParam( FOnRequestRemove, const TSharedRef<SFilter>& /*FilterToRemove*/ );
		DECLARE_DELEGATE_OneParam( FOnRequestRemoveAllButThis, const TSharedRef<SFilter>& /*FilterToKeep*/ );
		DECLARE_DELEGATE_OneParam( FOnRequestEnableOnly, const TSharedRef<SFilter>& /*FilterToEnable*/ );
		DECLARE_DELEGATE( FOnRequestEnableAll );
		DECLARE_DELEGATE( FOnRequestDisableAll );
		DECLARE_DELEGATE( FOnRequestRemoveAll );

		SLATE_BEGIN_ARGS( SFilter ){}

			/** If this is an front end filter, this is the filter object */
			SLATE_ARGUMENT( TSharedPtr<FFilterBase<FilterType>>, FrontendFilter )

			/** Invoked when the filter toggled */
			SLATE_EVENT( SBasicFilterBar<FilterType>::FOnFilterChanged, OnFilterChanged )

			/** Invoked when a request to remove this filter originated from within this filter */
			SLATE_EVENT( FOnRequestRemove, OnRequestRemove )

			/** Invoked when a request to enable only this filter originated from within this filter */
			SLATE_EVENT( FOnRequestEnableOnly, OnRequestEnableOnly )

			/** Invoked when a request to enable all filters originated from within this filter */
			SLATE_EVENT(FOnRequestEnableAll, OnRequestEnableAll)

			/** Invoked when a request to disable all filters originated from within this filter */
			SLATE_EVENT( FOnRequestDisableAll, OnRequestDisableAll )

			/** Invoked when a request to remove all filters originated from within this filter */
			SLATE_EVENT( FOnRequestRemoveAll, OnRequestRemoveAll )

			/** Invoked when a request to remove all filters originated from within this filter */
			SLATE_EVENT( FOnRequestRemoveAllButThis, OnRequestRemoveAllButThis )

			/** Determines how each individual filter pill looks like */
			SLATE_ARGUMENT(EFilterPillStyle, FilterPillStyle)

		SLATE_END_ARGS()

		/** Constructs this widget with InArgs */
		void Construct( const FArguments& InArgs )
		{
			bEnabled = false;
			OnFilterChanged = InArgs._OnFilterChanged;
			OnRequestRemove = InArgs._OnRequestRemove;
			OnRequestEnableOnly = InArgs._OnRequestEnableOnly;
			OnRequestEnableAll = InArgs._OnRequestEnableAll;
			OnRequestDisableAll = InArgs._OnRequestDisableAll;
			OnRequestRemoveAll = InArgs._OnRequestRemoveAll;
			OnRequestRemoveAllButThis = InArgs._OnRequestRemoveAllButThis;
			FrontendFilter = InArgs._FrontendFilter;

			// Get the tooltip and color of the type represented by this filter
			FilterColor = FLinearColor::White;
			if ( FrontendFilter.IsValid() )
			{
				FilterColor = FrontendFilter->GetColor();
				FilterToolTip = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(FrontendFilter.ToSharedRef(), &FFilterBase<FilterType>::GetToolTipText));
			}

			Construct_Internal(InArgs._FilterPillStyle);
		}

		/** Sets whether or not this filter is applied to the combined filter */
		void SetEnabled(bool InEnabled, bool InExecuteOnFilterChanged = true)
		{
			if ( InEnabled != bEnabled)
			{
				bEnabled = InEnabled;
				if (InExecuteOnFilterChanged)
				{
					OnFilterChanged.ExecuteIfBound();
				}
			}
		}

		/** Returns true if this filter contributes to the combined filter */
		bool IsEnabled() const
		{
			return bEnabled;
		}
		

		/** If this is an front end filter, this is the filter object */
		const TSharedPtr<FFilterBase<FilterType>>& GetFrontendFilter() const
		{
			return FrontendFilter;
		}

		/** Returns the display name for this filter */
		virtual FText GetFilterDisplayName() const
		{
			FText FilterName;
			if (FrontendFilter.IsValid())
			{
				FilterName = FrontendFilter->GetDisplayName();
			}

			if (FilterName.IsEmpty())
			{
				FilterName = LOCTEXT("UnknownFilter", "???");
			}

			return FilterName;
		}
		
		virtual FString GetFilterName() const
		{
			FString FilterName;
			if (FrontendFilter.IsValid())
			{
				FilterName = FrontendFilter->GetName();
			}
			return FilterName;
		}

	protected:

		/** Function that constructs the actual widget for subclasses to call */
		void Construct_Internal(EFilterPillStyle InFilterPillStyle)
		{
			TSharedPtr<SWidget> ContentWidget;

			switch(InFilterPillStyle)
			{
			case EFilterPillStyle::Basic:
				ContentWidget = SAssignNew( ToggleButtonPtr, SFilterCheckBox )
					.Style(FAppStyle::Get(), "FilterBar.BasicFilterButton")
					.ToolTipText(FilterToolTip)
					.IsChecked(this, &SFilter::IsChecked)
					.OnCheckStateChanged(this, &SFilter::FilterToggled)
					.CheckBoxContentUsesAutoWidth(false)
					.OnGetMenuContent(this, &SFilter::GetRightClickMenuContent)
					[
						SNew(STextBlock)
						.Margin(FMargin(0.0f))
						.TextStyle(FAppStyle::Get(), "SmallText")
						.Text(this, &SFilter::GetFilterDisplayName)
					];
				break;
			case EFilterPillStyle::Default:
			default:
				ContentWidget = SAssignNew( ToggleButtonPtr, SFilterCheckBox )
					.Style(FAppStyle::Get(), "FilterBar.FilterButton")
					.ToolTipText(FilterToolTip)
					.IsChecked(this, &SFilter::IsChecked)
					.OnCheckStateChanged(this, &SFilter::FilterToggled)
					.CheckBoxContentUsesAutoWidth(false)
					.OnGetMenuContent(this, &SFilter::GetRightClickMenuContent)
					[
						SNew(SBorder)
						.Padding(1.0f)
						.BorderImage(FAppStyle::Get().GetBrush("FilterBar.FilterBackground"))
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("FilterBar.FilterImage"))
								.ColorAndOpacity(this, &SFilter::GetFilterImageColorAndOpacity)
							]
							+SHorizontalBox::Slot()
							.Padding(TAttribute<FMargin>(this, &SFilter::GetFilterNamePadding))
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(this, &SFilter::GetFilterDisplayName)
								.IsEnabled_Lambda([this] {return bEnabled;})
							]
						]
					];
					
			}
			
			ChildSlot
			[
				ContentWidget.ToSharedRef()
			];

			ToggleButtonPtr->SetOnFilterCtrlClicked(FOnClicked::CreateSP(this, &SFilter::FilterCtrlClicked));
			ToggleButtonPtr->SetOnFilterAltClicked(FOnClicked::CreateSP(this, &SFilter::FilterAltClicked));
			ToggleButtonPtr->SetOnFilterDoubleClicked( FOnClicked::CreateSP(this, &SFilter::FilterDoubleClicked) );
			ToggleButtonPtr->SetOnFilterMiddleButtonClicked( FOnClicked::CreateSP(this, &SFilter::FilterMiddleButtonClicked) );
		}
		
		/** Handler for when the filter checkbox is clicked */
		void FilterToggled(ECheckBoxState NewState)
		{
			bEnabled = NewState == ECheckBoxState::Checked;
			OnFilterChanged.ExecuteIfBound();
		}

		/** Handler for when the filter checkbox is clicked and a control key is pressed */
		FReply FilterCtrlClicked()
		{
			OnRequestEnableAll.ExecuteIfBound();
			return FReply::Handled();
		}

		/** Handler for when the filter checkbox is clicked and an alt key is pressed */
		FReply FilterAltClicked()
		{
			OnRequestDisableAll.ExecuteIfBound();
			return FReply::Handled();
		}

		/** Handler for when the filter checkbox is double clicked */
		FReply FilterDoubleClicked()
		{
			// Disable all other filters and enable this one.
			OnRequestDisableAll.ExecuteIfBound();
			bEnabled = true;
			OnFilterChanged.ExecuteIfBound();

			return FReply::Handled();
		}

		/** Handler for when the filter checkbox is middle button clicked */
		FReply FilterMiddleButtonClicked()
		{
			RemoveFilter();
			return FReply::Handled();
		}

		/** Handler to create a right click menu */
		TSharedRef<SWidget> GetRightClickMenuContent()
		{
			constexpr bool bShouldCloseWindowAfterMenuSelection = true;
			// CloseSelfOnly in case the FilterBar is part of another menu
			constexpr bool bCloseSelfOnly = true;
			FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr, {}, bCloseSelfOnly);

			MenuBuilder.BeginSection("FilterOptions", LOCTEXT("FilterContextHeading", "Filter Options"));
			{
				MenuBuilder.AddMenuEntry(
					FText::Format( LOCTEXT("RemoveFilter", "Remove: {0}"), GetFilterDisplayName() ),
					LOCTEXT("RemoveFilterTooltip", "Remove this filter from the list. It can be added again in the filters menu."),
					FSlateIcon(),
					FUIAction( FExecuteAction::CreateSP(this, &SFilter::RemoveFilter) )
					);

				MenuBuilder.AddMenuEntry(
					FText::Format( LOCTEXT("EnableOnlyThisFilter", "Enable Only This: {0}"), GetFilterDisplayName() ),
					LOCTEXT("EnableOnlyThisFilterTooltip", "Enable only this filter from the list."),
					FSlateIcon(),
					FUIAction( FExecuteAction::CreateSP(this, &SFilter::EnableOnly) )
					);

			}
			MenuBuilder.EndSection();

			MenuBuilder.BeginSection("FilterBulkOptions", LOCTEXT("BulkFilterContextHeading", "Bulk Filter Options"));
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("EnableAllFilters", "Enable All Filters"),
					LOCTEXT("EnableAllFiltersTooltip", "Enables all filters."),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SFilter::EnableAllFilters))
				);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("DisableAllFilters", "Disable All Filters"),
					LOCTEXT("DisableAllFiltersTooltip", "Disables all active filters."),
					FSlateIcon(),
					FUIAction( FExecuteAction::CreateSP(this, &SFilter::DisableAllFilters) )
					);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("RemoveAllFilters", "Remove All Filters"),
					LOCTEXT("RemoveAllFiltersTooltip", "Removes all filters from the list."),
					FSlateIcon(),
					FUIAction( FExecuteAction::CreateSP(this, &SFilter::RemoveAllFilters) )
					);

				MenuBuilder.AddMenuEntry(
					FText::Format( LOCTEXT("RemoveAllButThisFilter", "Remove All But This: {0}"), GetFilterDisplayName() ),
					LOCTEXT("RemoveAllButThisFilterTooltip", "Remove all other filters except this one from the list."),
					FSlateIcon(),
					FUIAction( FExecuteAction::CreateSP(this, &SFilter::RemoveAllButThis) )
					);
			}
			MenuBuilder.EndSection();

			if (FrontendFilter.IsValid())
			{
				FrontendFilter->ModifyContextMenu(MenuBuilder);
			}

			return MenuBuilder.MakeWidget();
		}

		/** Removes this filter from the filter list */
		void RemoveFilter()
		{
			TSharedRef<SFilter> Self = SharedThis(this);
			OnRequestRemove.ExecuteIfBound( Self );
		}

		/** Remove all but this filter from the filter list. */
		void RemoveAllButThis()
		{
			TSharedRef<SFilter> Self = SharedThis(this);
			OnRequestRemoveAllButThis.ExecuteIfBound(Self);
		}

		/** Enables only this filter from the filter list */
		void EnableOnly()
		{
			TSharedRef<SFilter> Self = SharedThis(this);
			OnRequestEnableOnly.ExecuteIfBound( Self );
		}

		/** Enables all filters in the list */
		void EnableAllFilters()
		{
			OnRequestEnableAll.ExecuteIfBound();
		}

		/** Disables all active filters in the list */
		void DisableAllFilters()
		{
			OnRequestDisableAll.ExecuteIfBound();
		}

		/** Removes all filters in the list */
		void RemoveAllFilters()
		{
			OnRequestRemoveAll.ExecuteIfBound();
		}

		/** Handler to determine the "checked" state of the filter checkbox */
		ECheckBoxState IsChecked() const
		{
			return bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		/** Handler to determine the color of the checkbox when it is checked */
		FSlateColor GetFilterImageColorAndOpacity() const
		{
			return bEnabled ? FilterColor : FAppStyle::Get().GetSlateColor("Colors.Recessed");
		}

		EVisibility GetFilterOverlayVisibility() const
		{
			return bEnabled ? EVisibility::Collapsed : EVisibility::HitTestInvisible;
		}

		/** Handler to determine the padding of the checkbox text when it is pressed */
		FMargin GetFilterNamePadding() const
		{
			return ToggleButtonPtr->IsPressed() ? FMargin(4,2,4,0) : FMargin(4,1,4,1);
		}

	protected:
		/** Invoked when the filter toggled */
		FOnFilterChanged OnFilterChanged;

		/** Invoked when a request to remove this filter originated from within this filter */
		FOnRequestRemove OnRequestRemove;

		/** Invoked when a request to enable only this filter originated from within this filter */
		FOnRequestEnableOnly OnRequestEnableOnly;

		/** Invoked when a request to enable all filters originated from within this filter */
		FOnRequestEnableAll OnRequestEnableAll;

		/** Invoked when a request to disable all filters originated from within this filter */
		FOnRequestDisableAll OnRequestDisableAll;

		/** Invoked when a request to remove all filters originated from within this filter */
		FOnRequestDisableAll OnRequestRemoveAll;

		/** Invoked when a request to remove all filters except this one originated from within this filter */
		FOnRequestRemoveAllButThis OnRequestRemoveAllButThis;

		/** true when this filter should be applied to the search */
		bool bEnabled;

		/** If this is an front end filter, this is the filter object */
		TSharedPtr<FFilterBase<FilterType>> FrontendFilter;

		/** The button to toggle the filter on or off */
		TSharedPtr<SFilterCheckBox> ToggleButtonPtr;

		/** The color of the checkbox for this filter */
		FLinearColor FilterColor;

		/** The tooltip for this filter */
		TAttribute<FText> FilterToolTip;
	};

public:

 	/** Returns true if any filters are applied */
 	bool HasAnyFilters() const
 	{
 		return Filters.Num() > 0;
 	}

 	/** Retrieve a specific filter */
 	TSharedPtr<FFilterBase<FilterType>> GetFilter(const FString& InName) const
 	{
 		for (const TSharedRef<FFilterBase<FilterType>>& Filter : AllFrontendFilters)
 		{
 			if (Filter->GetName() == InName)
 			{
 				return Filter;
 			}
 		}
 		return TSharedPtr<FFilterBase<FilterType>>();
 	}

	/** Use this function to get all currently active filters (to filter your items)
	 * Not const on purpose: Subclasses might need to update the filter collection before getting it
	 */
	virtual TSharedPtr< TFilterCollection<FilterType> > GetAllActiveFilters()
 	{
 		return ActiveFilters;	
 	}

	/** Enable all the filters that are currently visible on the filter bar */
	void EnableAllFilters()
 	{
 		for (const TSharedRef<SFilter>& Filter : Filters)
 		{
 			Filter->SetEnabled(true, false);
 			if (const TSharedPtr<FFilterBase<FilterType>>& FrontendFilter = Filter->GetFrontendFilter())
 			{
 				SetFrontendFilterActive(FrontendFilter.ToSharedRef(), true);
 			}
 		}

 		OnFilterChanged.ExecuteIfBound();
 	}

	/** Disable all the filters that are currently visible on the filter bar */
	void DisableAllFilters()
 	{
 		for (const TSharedRef<SFilter>& Filter : Filters)
 		{
 			Filter->SetEnabled(false, false);
 			if (const TSharedPtr<FFilterBase<FilterType>>& FrontendFilter = Filter->GetFrontendFilter())
 			{
 				SetFrontendFilterActive(FrontendFilter.ToSharedRef(), false);
 			}
 		}

 		OnFilterChanged.ExecuteIfBound();
 	}

	/** Remove all filters from the filter bar, while disabling any active ones */
	virtual void RemoveAllFilters()
 	{
 		if (HasAnyFilters())
 		{
 			// Update the frontend filters collection
 			for (const TSharedRef<SFilter>& FilterToRemove : Filters)
 			{
 				if (const TSharedPtr<FFilterBase<FilterType>>& FrontendFilter = FilterToRemove->GetFrontendFilter())
 				{
 					SetFrontendFilterActive(FrontendFilter.ToSharedRef(), false); // Deactivate.
 				}

 				RemoveWidgetFromLayout(FilterToRemove);
 			}

 			Filters.Empty();
 			
 			// Notify that a filter has changed
 			OnFilterChanged.ExecuteIfBound();
 		}
 	}
	
 	/** Set the check box state of the specified filter (in the filter drop down) and pin/unpin a filter widget on/from the filter bar. When a filter is pinned (was not already pinned), it is activated and deactivated when unpinned. */
 	void SetFilterCheckState(const TSharedPtr<FFilterBase<FilterType>>& InFilter, ECheckBoxState InCheckState)
 	{
 		if (!InFilter || InCheckState == ECheckBoxState::Undetermined)
 		{
 			return;
 		}

 		// Check if the filter is already checked.
 		TSharedRef<FFilterBase<FilterType>> Filter = InFilter.ToSharedRef();
 		bool FrontendFilterChecked = IsFrontendFilterInUse(Filter);

 		if (InCheckState == ECheckBoxState::Checked && !FrontendFilterChecked)
 		{
 			AddFilterToBar(Filter)->SetEnabled(true); // Pin a filter widget on the UI and activate the filter. Same behaviour as FrontendFilterClicked()
 		}
 		else if (InCheckState == ECheckBoxState::Unchecked && FrontendFilterChecked)
 		{
 			RemoveFilter(Filter); // Unpin the filter widget and deactivate the filter.
 		}
 		// else -> Already in the desired 'check' state.
 	}

 	/** Returns the check box state of the specified filter (in the filter drop down). This tells whether the filter is pinned or not on the filter bar, but not if filter is active or not. @see IsFilterActive(). */
 	ECheckBoxState GetFilterCheckState(const TSharedPtr<FFilterBase<FilterType>>& InFilter) const
 	{
 		return InFilter && IsFrontendFilterInUse(InFilter.ToSharedRef()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
 	}

 	/** Returns true if the specified filter is both checked (pinned on the filter bar) and active (contributing to filter the result). */
 	bool IsFilterActive(const TSharedPtr<FFilterBase<FilterType>>& InFilter) const
 	{
 		if (InFilter.IsValid())
 		{
 			for (const TSharedRef<SFilter>& Filter : Filters)
 			{
 				if (InFilter == Filter->GetFrontendFilter())
 				{
 					return Filter->IsEnabled(); // Is active or not?
 				}
 			}
 		}
 		return false;
 	}

	/** Are we sorting filters alphabetically */
    bool GetSortFiltersAlphabetically() const
    {
 		return bSortFiltersAlphabetically;
    }

	/** Set filtering method for filters */
	void SetSortFiltersAlphabetically(bool bInSort)
    {
        if (bSortFiltersAlphabetically != bInSort)
        {
        	bSortFiltersAlphabetically = bInSort;
        	RequestFiltersUpdate();
        }
    }

protected:

	/** Overlay for filter icon, badge if there are any active filters */
	const FSlateBrush* GetFilterBadgeIcon() const
	{
		for (const TSharedRef<SFilter>& Filter : Filters)
		{
			if(Filter->IsEnabled())
			{
				return FAppStyle::Get().GetBrush("Icons.BadgeModified");
			}
		}

		return nullptr;
	}
	
	/** Remove all filters except the specified one */
	virtual void RemoveAllButThis(const TSharedRef<SFilter>& FilterToKeep)
	{
		for (const TSharedRef<SFilter>& Filter : Filters)
		{
			RemoveWidgetFromLayout(Filter);
			
			if (Filter == FilterToKeep)
			{
				continue;
			}

			if (const TSharedPtr<FFilterBase<FilterType>>& FrontendFilter = Filter->GetFrontendFilter())
			{
				SetFrontendFilterActive(FrontendFilter.ToSharedRef(), false);
			}
		}

		Filters.Empty();

		AddFilterToBar(FilterToKeep);

		OnFilterChanged.ExecuteIfBound();
	}

	/** Add a widget to the current filter layout */
	void AddWidgetToLayout(const TSharedRef<SWidget> WidgetToAdd)
	{
		FMargin HorizontalFilterPadding;
		
		switch(FilterPillStyle)
		{
		case EFilterPillStyle::Basic:
			HorizontalFilterPadding = FMargin(2);
			break;
		case EFilterPillStyle::Default:
		default:
			HorizontalFilterPadding = FMargin(3);
		}
			
		
		if(FilterBarLayout == EFilterBarLayout::Horizontal)
		{
			HorizontalFilterBox->AddSlot()
			.Padding(HorizontalFilterPadding)
			[
				WidgetToAdd
			];
		}
		else
		{
			VerticalFilterBox->AddSlot()
			.Padding(4, 2)
			[
				WidgetToAdd
			];
		}
	}

	/** Remove a widget from the current filter layout */
	void RemoveWidgetFromLayout(const TSharedRef<SWidget> WidgetToRemove)
	{
		if(FilterBarLayout == EFilterBarLayout::Horizontal)
		{
			HorizontalFilterBox->RemoveSlot(WidgetToRemove);
		}
		else
		{
			VerticalFilterBox->RemoveSlot(WidgetToRemove);
		}
	}
	
 	/** Sets the active state of a frontend filter. */
 	void SetFrontendFilterActive(const TSharedRef<FFilterBase<FilterType>>& Filter, bool bActive)
	{
		Filter->SetActiveInCollection(Filter, bActive, *ActiveFilters);
	}

	/* 'Activate' A filter by adding it to the filter bar, does not turn it on */
	TSharedRef<SFilter> AddFilterToBar(const TSharedRef<FFilterBase<FilterType>>& Filter)
	{
		TSharedRef<SFilter> NewFilter =
			SNew(SFilter)
			.FrontendFilter(Filter)
			.FilterPillStyle(FilterPillStyle)
			.OnFilterChanged( this, &SBasicFilterBar<FilterType>::FrontendFilterChanged, Filter )
			.OnRequestRemove(this, &SBasicFilterBar<FilterType>::RemoveFilterAndUpdate)
			.OnRequestEnableOnly(this, &SBasicFilterBar<FilterType>::EnableOnlyThisFilter)
			.OnRequestEnableAll(this, &SBasicFilterBar<FilterType>::EnableAllFilters)
			.OnRequestDisableAll(this, &SBasicFilterBar<FilterType>::DisableAllFilters)
			.OnRequestRemoveAll(this, &SBasicFilterBar<FilterType>::RemoveAllFilters)
			.OnRequestRemoveAllButThis(this, &SBasicFilterBar<FilterType>::RemoveAllButThis);

		AddFilterToBar( NewFilter );

		return NewFilter;
	}

	/* 'Activate' A filter by adding it to the filter bar, does not turn it on */
	void AddFilterToBar(const TSharedRef<SFilter>& FilterToAdd)
	{
		Filters.Add(FilterToAdd);

		if (bSortFiltersAlphabetically)
		{
			// Update next tick in case this is called in a loop
			RequestFiltersUpdate();
		}
		else
		{
			// Add new filter to the end directly
			AddWidgetToLayout(FilterToAdd);
		}
	}

	/** Updates filters in the layout if requested */
	virtual void Tick(const FGeometry& InGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		if (bRequestFiltersUpdate)
		{
			bRequestFiltersUpdate = false;

			HorizontalFilterBox->ClearChildren();
			VerticalFilterBox->ClearChildren();

			auto AddFiltersToLayout = [this](const TArray<TSharedRef<SFilter>>& InFilters)
				{
					for (const TSharedRef<SFilter>& Filter : InFilters)
					{
						AddWidgetToLayout(Filter);
					}
				};

			if (bSortFiltersAlphabetically)
			{
				TArray<TSharedRef<SFilter>> FiltersSorted = Filters;

				FiltersSorted.Sort(
					[](const TSharedRef<SFilter>& A, const TSharedRef<SFilter>& B)
					{
						return A->GetFilterDisplayName().CompareTo(B->GetFilterDisplayName()) < 0;
					}
				);

				AddFiltersToLayout(FiltersSorted);
			}
			else
			{
				AddFiltersToLayout(Filters);
			}
		}

		SCompoundWidget::Tick(InGeometry, InCurrentTime, InDeltaTime);
	}

	/** Handler for when the enable only this button was clicked on a single filter */
	void EnableOnlyThisFilter(const TSharedRef<SFilter>& FilterToEnable)
	{
		for (const TSharedRef<SFilter>& Filter : Filters)
		{
			bool bEnable = Filter == FilterToEnable;
			Filter->SetEnabled(bEnable, /*ExecuteOnFilterChange*/false);
			if (const TSharedPtr<FFilterBase<FilterType>>& FrontendFilter = Filter->GetFrontendFilter())
			{
				SetFrontendFilterActive(FrontendFilter.ToSharedRef(), bEnable);
			}
		}

		OnFilterChanged.ExecuteIfBound();
	}

	/** Remove a filter from the filter bar */
	void RemoveFilter(const TSharedRef<FFilterBase<FilterType>>& InFilterToRemove, bool ExecuteOnFilterChanged= true)
	{
		TSharedPtr<SFilter> FilterToRemove;
		for (const TSharedRef<SFilter>& Filter : Filters)
		{
			if (Filter->GetFrontendFilter() == InFilterToRemove)
			{
				FilterToRemove = Filter;
				break;
			}
		}

		if (FilterToRemove.IsValid())
		{
			if (ExecuteOnFilterChanged)
			{
				RemoveFilterAndUpdate(FilterToRemove.ToSharedRef());
			}
			else
			{
				RemoveFilter(FilterToRemove.ToSharedRef());
			}
		}
	}

	/** Remove a filter from the filter bar */
	virtual void RemoveFilter(const TSharedRef<SFilter>& FilterToRemove)
	{
		Filters.Remove(FilterToRemove);

		if (const TSharedPtr<FFilterBase<FilterType>>& FrontendFilter = FilterToRemove->GetFrontendFilter()) // Is valid?
		{
			// Update the frontend filters collection
			SetFrontendFilterActive(FrontendFilter.ToSharedRef(), false);
		}

		RemoveWidgetFromLayout(FilterToRemove);
	}

	/** Remove a filter from the filter bar */
	void RemoveFilterAndUpdate(const TSharedRef<SFilter>& FilterToRemove)
	{
		RemoveFilter(FilterToRemove);

		// Notify that a filter has changed
		OnFilterChanged.ExecuteIfBound();
	}

 	/** Handler for when a frontend filter state has changed */
 	void FrontendFilterChanged(TSharedRef<FFilterBase<FilterType>> FrontendFilter)
	{
		TSharedPtr<SFilter> FilterToUpdate;
		for (const TSharedRef<SFilter>& Filter : Filters)
		{
			if (Filter->GetFrontendFilter() == FrontendFilter)
			{
				FilterToUpdate = Filter;
				break;
			}
		}

		if (FilterToUpdate.IsValid())
		{
			SetFrontendFilterActive(FrontendFilter, FilterToUpdate->IsEnabled());
			OnFilterChanged.ExecuteIfBound();
		}
	}

 	/** Handler for when the add filter menu is populated by a non-category */
 	void CreateOtherFiltersMenuCategory(FToolMenuSection& Section, TSharedPtr<FFilterCategory> MenuCategory) const
	{
		for (const TSharedRef<FFilterBase<FilterType>>& FrontendFilter : AllFrontendFilters)
		{
			if(FrontendFilter->GetCategory() == MenuCategory)
			{
				Section.AddMenuEntry(
					NAME_None,
					FrontendFilter->GetDisplayName(),
					FrontendFilter->GetToolTipText(),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), FrontendFilter->GetIconName()),
					FUIAction(
					FExecuteAction::CreateSP(const_cast< SBasicFilterBar<FilterType>* >(this), &SBasicFilterBar<FilterType>::FrontendFilterClicked, FrontendFilter),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SBasicFilterBar<FilterType>::IsFrontendFilterInUse, FrontendFilter)),
					EUserInterfaceActionType::ToggleButton
					);
			}
		}
	}

	/** Handler for when the add filter menu is populated by a non-category */
 	void CreateOtherFiltersMenuCategory(UToolMenu* InMenu, TSharedPtr<FFilterCategory> MenuCategory) const
	{
		CreateOtherFiltersMenuCategory(InMenu->AddSection("Section"), MenuCategory);
	}

	/** Handler for a frontend filter is clicked */
 	void FrontendFilterClicked(TSharedRef<FFilterBase<FilterType>> FrontendFilter)
	{
		if (IsFrontendFilterInUse(FrontendFilter))
		{
			RemoveFilter(FrontendFilter);
		}
		else
		{
			TSharedRef<SFilter> NewFilter = AddFilterToBar(FrontendFilter);
			NewFilter->SetEnabled(true);
		}
	}

	/** Handler to check if a frontend filter is in use */
 	bool IsFrontendFilterInUse(TSharedRef<FFilterBase<FilterType>> FrontendFilter) const
	{
		for (const TSharedRef<SFilter>& Filter : Filters)
		{
			if (Filter->GetFrontendFilter() == FrontendFilter)
			{
				return true;
			}
		}

		return false;
	}

	/** Handler for when a filter category is clicked */
 	void FrontendFilterCategoryClicked(TSharedPtr<FFilterCategory> MenuCategory)
	{
		bool bFullCategoryInUse = IsFrontendFilterCategoryInUse(MenuCategory);
		bool ExecuteOnFilterChanged = false;

		for (const TSharedRef<FFilterBase<FilterType>>& FrontendFilter : AllFrontendFilters)
		{
			if (FrontendFilter->GetCategory() == MenuCategory)
			{
				if (bFullCategoryInUse)
				{
					RemoveFilter(FrontendFilter, false);
					ExecuteOnFilterChanged = true;
				}
				else if (!IsFrontendFilterInUse(FrontendFilter))
				{
					TSharedRef<SFilter> NewFilter = AddFilterToBar(FrontendFilter);
					NewFilter->SetEnabled(true, false);
					SetFrontendFilterActive(FrontendFilter, NewFilter->IsEnabled());
					ExecuteOnFilterChanged = true;
				}
			}
		}

		if (ExecuteOnFilterChanged)
		{
			OnFilterChanged.ExecuteIfBound();
		}
	}
	
	/** Handler to check if a filter category is in use */
 	bool IsFrontendFilterCategoryInUse(TSharedPtr<FFilterCategory> MenuCategory) const
	{
		for (const TSharedRef<FFilterBase<FilterType>>& FrontendFilter : AllFrontendFilters)
		{
			if (FrontendFilter->GetCategory() == MenuCategory && !IsFrontendFilterInUse(FrontendFilter))
			{
				return false;
			}
		}

		return true;
	}
	
	/** Handler to determine the "checked" state of a frontend filter category in the filter dropdown */
    ECheckBoxState IsFrontendFilterCategoryChecked(TSharedPtr<FFilterCategory> MenuCategory) const
    {
    	bool bIsAnyActionInUse = false;
    	bool bIsAnyActionNotInUse = false;
		
		for (const TSharedRef<FFilterBase<FilterType>>& FrontendFilter : AllFrontendFilters)
		{
			if (FrontendFilter->GetCategory() == MenuCategory)
			{
				if (IsFrontendFilterInUse(FrontendFilter))
				{
					bIsAnyActionInUse = true;
				}
				else
				{
					bIsAnyActionNotInUse = true;
				}

				if (bIsAnyActionInUse && bIsAnyActionNotInUse)
				{
					return ECheckBoxState::Undetermined;
				}
			}
		}

    	if (bIsAnyActionInUse)
    	{
    		return ECheckBoxState::Checked;
    	}
    	else
    	{
    		return ECheckBoxState::Unchecked;
    	}
    }

 	/** Called when reset filters option is pressed */
 	void OnResetFilters()
	{
		RemoveAllFilters();
	}

 	/** Called externally to check/pin a filter, and activates/deactivates it */
 	void OnSetFilterActive(bool bActive, TWeakPtr<FFilterBase<FilterType>> InWeakFilter)
 	{
 		TSharedPtr<FFilterBase<FilterType>> Filter = InWeakFilter.Pin();
 		if (Filter.IsValid())
 		{
 			if (!IsFrontendFilterInUse(Filter.ToSharedRef()))
 			{
 				TSharedRef<SFilter> NewFilter = AddFilterToBar(Filter.ToSharedRef());
 				NewFilter->SetEnabled(bActive);
 			}
 			else
 			{
 				for (const TSharedRef<SFilter>& PinnedFilter : Filters)
 				{
 					if (PinnedFilter->GetFrontendFilter() == Filter)
 					{
 						PinnedFilter->SetEnabled(bActive);
 						break;
 					}
 				}
 			}
 		}
 	}

	/** Called externally to determine if a filter has been checked/pinned and activated */
	bool OnIsFilterActive(TWeakPtr<FFilterBase<FilterType>> InWeakFilter)
	{
		TSharedPtr<FFilterBase<FilterType>> Filter = InWeakFilter.Pin();
		if(Filter.IsValid())
		{
			if(!IsFrontendFilterInUse(Filter.ToSharedRef()))
			{
				return false;
			}
			else
			{
				return IsFilterActive(Filter);
			}
		}

		return false;
	}
 	
 	/** Handler for when a checkbox next to a custom text filter is clicked */
	void CustomTextFilterClicked(ECheckBoxState CheckBoxState, TSharedRef<ICustomTextFilter<FilterType>> Filter)
    {
		TSharedRef<FFilterBase<FilterType>> FrontendFilter = Filter->GetFilter().ToSharedRef();
    	if (CheckBoxState == ECheckBoxState::Unchecked)
    	{
    		RemoveFilter(FrontendFilter);
    	}
    	else
    	{
    		TSharedRef<SFilter> NewFilter = AddFilterToBar(FrontendFilter);
    		NewFilter->SetEnabled(true);
    	}
    }

	/** Handler for when a custom text filter is created */
	virtual void OnCreateCustomTextFilter(const FCustomTextFilterData& InFilterData, bool bApplyFilter)
	{
		// Create the text filter using the given delegate
		TSharedRef<ICustomTextFilter<FilterType>> NewTextFilter = CreateTextFilter.Execute().ToSharedRef();
		
		// Fill in the data the widget gives us
		NewTextFilter->SetFromCustomTextFilterData(InFilterData);

		TSharedPtr<FFilterBase<FilterType>> NewFilter = NewTextFilter->GetFilter();

		if(!NewFilter)
		{
			// Close the create custom filter dialog 		
			OnCancelCustomTextFilterDialog();
			return; // TODO: error message? Some way to ensure the user always implements NewTextFilter->GetFilter()?
		}
		
		CustomTextFilters.Add(NewTextFilter);
		
		TSharedRef<SFilter> AddedFilter = AddFilterToBar(NewFilter.ToSharedRef());
		
		AddedFilter->SetEnabled(bApplyFilter);
		SetFrontendFilterActive(NewFilter.ToSharedRef(), bApplyFilter);

		// Close the create custom filter dialog 		
		OnCancelCustomTextFilterDialog();
	}

	/** Handler for when a custom text filter is modified */
	virtual void OnModifyCustomTextFilter(const FCustomTextFilterData& InFilterData, TSharedPtr<ICustomTextFilter<FilterType>> InFilter)
	{
		// Update the filter with the data the widget has given us
		if(InFilter)
		{
			InFilter->SetFromCustomTextFilterData(InFilterData);
		}

		TSharedPtr<FFilterBase<FilterType>> NewFilter = InFilter->GetFilter();
		
		// If the filter we modified is active in the bar, remove and re-add it to update the data
		for (const TSharedRef<SFilter>& Filter : Filters)
		{
			if (Filter->GetFrontendFilter() == NewFilter)
			{
				bool bWasEnabled = Filter->IsEnabled();
				
				RemoveFilter(NewFilter.ToSharedRef());

				TSharedRef<SFilter> AddedFilter = AddFilterToBar(NewFilter.ToSharedRef());
				
				AddedFilter->SetEnabled(bWasEnabled);
				SetFrontendFilterActive(NewFilter.ToSharedRef(), bWasEnabled);
			}
		}

		OnCancelCustomTextFilterDialog();
	}

	/** Handler for when a custom text filter is deleted */
	virtual void OnDeleteCustomTextFilter(const TSharedPtr<ICustomTextFilter<FilterType>> InFilter)
	{
		if(InFilter)
		{
			CustomTextFilters.RemoveSingle(InFilter.ToSharedRef());

			TSharedPtr<FFilterBase<FilterType>> NewFilter = InFilter->GetFilter();
			
			RemoveFilter(NewFilter.ToSharedRef());

			// We fire this delegate here, since a CustomTextFilter was removed from the list 
			OnFilterChanged.ExecuteIfBound();
		}
		
		OnCancelCustomTextFilterDialog();
	}

	/** Handler to close the custom text filter dialog */
	virtual void OnCancelCustomTextFilterDialog()
	{
		if(CustomTextFilterWindow.IsValid())
		{
			CustomTextFilterWindow.Pin()->RequestDestroyWindow();
		}
	}

	void GetAllCustomTextFilterLabels(TArray<FText>& OutFilterLabels)
	{
		for (const TSharedRef<ICustomTextFilter<FilterType>>& Filter : CustomTextFilters)
		{
			FCustomTextFilterData FilterData = Filter->CreateCustomTextFilterData();
			OutFilterLabels.Add(FilterData.FilterLabel);
		}
	}

	/** Creates a dialog box with the SCustomTextFilterDialog Widget */
	void CreateCustomTextFilterWindow(const FCustomTextFilterData& CustomTextFilterData, TSharedPtr<ICustomTextFilter<FilterType>> InFilter)
	{
		/** If we already have a window, delete it */
		OnCancelCustomTextFilterDialog();

		// If we have a filter to edit, we are in edit mode
		bool bInEditMode = InFilter.IsValid();
		
		FText WindowTitle = bInEditMode ? LOCTEXT("ModifyCustomTextFilterWindow", "Modify Custom Filter") : LOCTEXT("CreateCustomTextFilterWindow", "Create Custom Filter");
		
		TSharedPtr<SWindow> NewTextFilterWindow = SNew(SWindow)
			.Title(WindowTitle)
			.HasCloseButton(true)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.SizingRule(ESizingRule::FixedSize)
			.ClientSize(FVector2D(724, 183));
		
		TSharedPtr<SCustomTextFilterDialog> CustomTextFilterDialog =
			SNew(SCustomTextFilterDialog)
			.FilterData(CustomTextFilterData)
			.InEditMode(bInEditMode)
			.InEnableColorEditing(FilterPillStyle != EFilterPillStyle::Basic)
			.OnCreateFilter(this, &SBasicFilterBar<FilterType>::OnCreateCustomTextFilter)
			.OnDeleteFilter(this, &SBasicFilterBar<FilterType>::OnDeleteCustomTextFilter, InFilter)
			.OnModifyFilter(this, &SBasicFilterBar<FilterType>::OnModifyCustomTextFilter, InFilter)
			.OnCancelClicked(this, &SBasicFilterBar<FilterType>::OnCancelCustomTextFilterDialog)
			.OnGetFilterLabels(this, &SBasicFilterBar<FilterType>::GetAllCustomTextFilterLabels);

		NewTextFilterWindow->SetContent(CustomTextFilterDialog.ToSharedRef());
		FSlateApplication::Get().AddWindow(NewTextFilterWindow.ToSharedRef());
		
		CustomTextFilterWindow = NewTextFilterWindow;
	}

	/** Creates a dialog box to Add a custom text filter */
	void CreateAddCustomTextFilterWindow()
	{
		CreateCustomTextFilterWindow(FCustomTextFilterData(), nullptr);
	}

	/** Creates a dialog box to Edit an existing custom text filter */
	void CreateEditCustomTextFilterWindow(TSharedPtr<ICustomTextFilter<FilterType>> InFilter)
	{
		if(!InFilter)
		{
			return;
		}

		FCustomTextFilterData CustomTextFilterData = InFilter->CreateCustomTextFilterData();

		CreateCustomTextFilterWindow(CustomTextFilterData, InFilter);
	}

	/** Creates a dialog to add a custom text filter from the given search text */
	void CreateCustomTextFilterFromSearch(const FText& InSearchText)
	{
		FCustomTextFilterData CustomTextFilterData;
		CustomTextFilterData.FilterLabel = InSearchText;
		CustomTextFilterData.FilterString = InSearchText;

		CreateCustomTextFilterWindow(CustomTextFilterData, nullptr);
	}
	
	/** Populate the Custom Filters submenu in the Add Filter dropdown */
	void CreateTextFiltersMenu(UToolMenu* InMenu)
	{
		FToolMenuSection& Section = InMenu->AddSection("FilterBarTextFiltersSubmenu");
		
		Section.AddMenuEntry(
			"CreateNewTextFilter",
			LOCTEXT("CreateNewTextFilter", "Create New Filter"),
			LOCTEXT("CreateNewTextFilterTooltip", "Create a new text filter"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.PlusCircle"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SBasicFilterBar<FilterType>::CreateAddCustomTextFilterWindow)
				)
		);

		FToolMenuSection& CustomFiltersSection = InMenu->AddSection("FilterBarCustomTextFiltersSection");
		
		for (const TSharedRef<ICustomTextFilter<FilterType>>& Filter : CustomTextFilters)
		{
			const TSharedRef<FFilterBase<FilterType>>& FrontendFilter = Filter->GetFilter().ToSharedRef();
			
			bool bIsFilterChecked = IsFrontendFilterInUse(FrontendFilter);

			TSharedPtr<SButton> CustomTextFilterWidget = 
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "Menu.Button")
				.ContentPadding(0.0f)
				.OnClicked_Lambda([this, &Filter]()
				{
					this->FrontendFilterClicked(Filter->GetFilter().ToSharedRef());
					FSlateApplication::Get().DismissAllMenus();
					return FReply::Handled();
				})
				[
					SNew(SHorizontalBox)

					// Checkbox for the filter
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(16, 0, 0, 0)
					[
						SNew(SCheckBox)
						.Style(FAppStyle::Get(), "Menu.CheckBox")
						.OnCheckStateChanged(FOnCheckStateChanged::CreateSP(this, &SBasicFilterBar<FilterType>::CustomTextFilterClicked, Filter))
						.IsChecked(bIsFilterChecked)
					]

					// A text block showing the filter name
					+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					.Padding(4, 0, 0, 0)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FrontendFilter->GetDisplayName())
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.Padding(0, 0, 16, 0)
					[
						SNew(SSimpleButton)
						.Icon(FAppStyle::GetBrush("Icons.Edit"))
						.OnClicked_Lambda([this, &Filter]()
						{
							this->CreateEditCustomTextFilterWindow(Filter);
							return FReply::Handled();
						})
					]
				];

			CustomFiltersSection.AddEntry(
			FToolMenuEntry::InitWidget(
				NAME_None,
				CustomTextFilterWidget.ToSharedRef(),
				FText::GetEmpty(),
				true,
				true
			));
		}
	}

	/** Helper function to add common sections to the Add Filter Menu */
	void PopulateCommonFilterSections(UToolMenu* Menu)
	{
		FToolMenuSection& Section = Menu->AddSection("FilterBarResetFilters");
		Section.AddMenuEntry(
			"ResetFilters",
			LOCTEXT("FilterListResetFilters", "Reset Filters"),
			LOCTEXT("FilterListResetToolTip", "Resets current filter selection"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "PropertyWindow.DiffersFromDefault"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SBasicFilterBar<FilterType>::OnResetFilters),
				FCanExecuteAction::CreateLambda([this]() { return HasAnyFilters(); }))
		);

		// Only add the custom text filter submenu if we have a valid CreateTextFilter delegate to use
		if(CreateTextFilter.IsBound())
		{
			FToolMenuSection& CustomFiltersSection = Menu->AddSection("FilterBarTextFilters");

			CustomFiltersSection.AddSubMenu(
					"CustomFiltersSubMenu",
					LOCTEXT("FilterBarTextFilters", "Custom Filters"),
					LOCTEXT("FilterBarTextFiltersTooltip", "Custom Filters"),
					FNewToolMenuDelegate::CreateSP(this, &SBasicFilterBar<FilterType>::CreateTextFiltersMenu)
					);
		}
	}

	/** Helper function to add all custom filters to the Add Filter Menu */
	void PopulateCustomFilters(UToolMenu* Menu)
	{
		if(bUseSectionsForCategories)
		{
			// Add all the filters as sections
			for (const TSharedPtr<FFilterCategory>& Category : AllFilterCategories)
			{
				FToolMenuSection& Section = Menu->AddSection(*Category->Title.ToString(), Category->Title);
				CreateOtherFiltersMenuCategory(Section, Category);
			}
		}
		else
		{
			FToolMenuSection& Section = Menu->AddSection("BasicFilterBarFiltersMenu", LOCTEXT("FilterBarOtherFiltersSection", "Other Filters"));
			
			// Add all the filters as submenus
			for (const TSharedPtr<FFilterCategory>& Category : AllFilterCategories)
			{
				Section.AddSubMenu(
					NAME_None,
					Category->Title,
					Category->Tooltip,
					FNewToolMenuDelegate::CreateSP(this, &SBasicFilterBar<FilterType>::CreateOtherFiltersMenuCategory, Category),
					FUIAction(
					FExecuteAction::CreateSP( this, &SBasicFilterBar<FilterType>::FrontendFilterCategoryClicked, Category ),
					FCanExecuteAction(),
					FGetActionCheckState::CreateSP(this, &SBasicFilterBar<FilterType>::IsFrontendFilterCategoryChecked, Category ) ),
					EUserInterfaceActionType::ToggleButton
					);
			}
		}
	}

	/** Request a filter list update next tick */
	void RequestFiltersUpdate()
	{
		bRequestFiltersUpdate = true;
	}

private:

	/** Handler for when the add filter button was clicked */
	virtual TSharedRef<SWidget> MakeAddFilterMenu()
	{
		const FName FilterMenuName = "FilterBar.FilterMenu";
		if (!UToolMenus::Get()->IsMenuRegistered(FilterMenuName))
		{
			UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(FilterMenuName);
			Menu->bShouldCloseWindowAfterMenuSelection = true;
			Menu->bCloseSelfOnly = true;

			Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				if (UFilterBarContext* Context = InMenu->FindContext<UFilterBarContext>())
				{
					Context->PopulateFilterMenu.ExecuteIfBound(InMenu);
					Context->OnExtendAddFilterMenu.ExecuteIfBound(InMenu);
				}
			}));
		}

		UFilterBarContext* FilterBarContext = NewObject<UFilterBarContext>();
		FilterBarContext->PopulateFilterMenu = FOnPopulateAddFilterMenu::CreateSP(this, &SBasicFilterBar<FilterType>::PopulateAddFilterMenu);
		FilterBarContext->OnExtendAddFilterMenu = OnExtendAddFilterMenu;
		FToolMenuContext ToolMenuContext(FilterBarContext);
		
		return UToolMenus::Get()->GenerateWidget(FilterMenuName, ToolMenuContext);
	}
	
	/** Function to populate the add filter menu */
	void PopulateAddFilterMenu(UToolMenu* Menu)
	{
		PopulateCommonFilterSections(Menu);
		
		PopulateCustomFilters(Menu);
	}

	/** Whether a refresh is requested next tick to re-render filters in the bar */
	bool bRequestFiltersUpdate = false;
	
protected:

	/** The horizontal wrap box which contains all the filters (used in the horizontal layout) */
	TSharedPtr<SWrapBox> HorizontalFilterBox;
	
	/** The vertical scroll box which contains all the filters (used in the vertical layout) */
	TSharedPtr<SScrollBox> VerticalFilterBox;
	
	/** The widget switcher containing the horizontal and vertical filter widgets */
	TSharedPtr<SWidgetSwitcher> FilterBox;

	/** The current Filter Layout being used */
	EFilterBarLayout FilterBarLayout;

	/** Delegate for when the layout is changed */
	FOnFilterBarLayoutChanging OnFilterBarLayoutChanging;
	
 	/** All SFilters in the list */
 	TArray< TSharedRef<SFilter> > Filters;

 	/** All possible filter objects */
 	TArray< TSharedRef< FFilterBase<FilterType> > > AllFrontendFilters;

	/** Currently active filter objects */
	TSharedPtr< TFilterCollection<FilterType> > ActiveFilters;

 	/** All filter categories (for menu construction) */
 	TArray< TSharedPtr<FFilterCategory> > AllFilterCategories;

 	/** Delegate for when filters have changed */
 	FOnFilterChanged OnFilterChanged;

	/** Delegate to extend the AddFilter Menu */
	FOnExtendAddFilterMenu OnExtendAddFilterMenu;

	/** A delegate used to create a TTextFilter for FilterType */
	FCreateTextFilter CreateTextFilter;
	
	/** Custom text filters that a user can create */
	TArray< TSharedRef< ICustomTextFilter<FilterType> > > CustomTextFilters;

	/** The window containing the custom text filter dialog */
	TWeakPtr<SWindow> CustomTextFilterWindow;

	/** Whether the orientation can be changed after initailization */
	bool bCanChangeOrientation;

	/** Whether to use submenus or sections for categories in the filter menu */
	bool bUseSectionsForCategories;

	/** Whether to pin all front end filters by default. Will add them from the Add Filter menu to the UI, but does not necessarily activate them. */
	bool bPinAllFrontendFilters;

	/** Determines how each individual pill looks like */
	EFilterPillStyle FilterPillStyle;

	/** Determines whether filters are sorted in the bar or appear in the order they are added */
	bool bSortFiltersAlphabetically;
	
	friend struct FFrontendFilterExternalActivationHelper<FilterType>;
};

/** Helper struct to avoid friending the whole of SBasicFilterBarSBasicFilterBar */
template<typename FilterType>
struct FFrontendFilterExternalActivationHelper
{
	static void BindToFilter(TSharedRef<SBasicFilterBar<FilterType>> InFilterList, TSharedRef<FFilterBase<FilterType>> InFrontendFilter)
	{
		TWeakPtr<FFilterBase<FilterType>> WeakFilter = InFrontendFilter;
		InFrontendFilter->SetActiveEvent.AddSP(&InFilterList.Get(), &SBasicFilterBar<FilterType>::OnSetFilterActive, WeakFilter);
		InFrontendFilter->IsActiveEvent.BindSP(&InFilterList.Get(), &SBasicFilterBar<FilterType>::OnIsFilterActive, WeakFilter);
	}
};

#undef LOCTEXT_NAMESPACE
