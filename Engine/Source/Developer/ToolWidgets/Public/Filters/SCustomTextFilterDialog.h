// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Filters/CustomTextFilters.h"

#define UE_API TOOLWIDGETS_API

class SColorBlock;
class SEditableTextBox;

class SCustomTextFilterDialog : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_TwoParams(FOnCreateFilter, const FCustomTextFilterData& /* InFilterData */, bool /* bApplyFilter */);
	DECLARE_DELEGATE(FOnDeleteFilter);
	DECLARE_DELEGATE_OneParam(FOnModifyFilter, const FCustomTextFilterData& /* InFilterData */);
	DECLARE_DELEGATE(FOnCancelClicked);
	DECLARE_DELEGATE_OneParam(FOnGetFilterLabels, TArray<FText> & /* FilterNames */);

	SLATE_BEGIN_ARGS(SCustomTextFilterDialog)
		: _InEnableColorEditing(true)
		{}
	
    	/** The filter that this dialog is creating/editing */
    	SLATE_ARGUMENT(FCustomTextFilterData, FilterData)

		/** True if we are editing an existing filter, false if we are creating a new one */
		SLATE_ARGUMENT(bool, InEditMode)

		/** True if we allow editing the filter color, mainly used to disable color editing on the basic pill style as it has no effect */
		SLATE_ARGUMENT(bool, InEnableColorEditing)
		
		/** Delegate for when the Create button is clicked */
		SLATE_EVENT(FOnCreateFilter, OnCreateFilter)
		
		/** Delegate for when the Delete button is clicked */
		SLATE_EVENT(FOnDeleteFilter, OnDeleteFilter)
		
		/** Delegate for when the Cancel button is clicked */
		SLATE_EVENT(FOnCancelClicked, OnCancelClicked)

		/** Delegate for when the Modify Filter button is clicked */
        SLATE_EVENT(FOnModifyFilter, OnModifyFilter)

		/** Delegate to get all existing filter labels to check for duplicates */
		SLATE_EVENT(FOnGetFilterLabels, OnGetFilterLabels)
    
    SLATE_END_ARGS()
    	
    /** Constructs this widget with InArgs */
    UE_API void Construct( const FArguments& InArgs );

protected:

	/* Handler for when the color block is clicked to open the color picker */
	UE_API FReply ColorBlock_OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	UE_API void HandleColorValueChanged(FLinearColor NewValue);
	
	UE_API FReply OnDeleteButtonClicked() const;
	
	UE_API FReply OnCreateFilterButtonClicked(bool bApplyFilter) const;
	
	UE_API FReply OnCancelButtonClicked() const;

	UE_API FReply OnModifyButtonClicked() const;

	UE_API bool CheckFilterValidity() const;
	
protected:

	/* True if we are editing a filter, false if we are creating a new filter */
	bool bInEditMode;

	/* True if we want to give the user the ability to pick a color for the filter in the edit/create menu */
	bool bEnableColorEditing;

	/* The current filter data we are editing */
	FCustomTextFilterData FilterData;
	
	/* The initial, unedited filter data we were provided */
	FCustomTextFilterData InitialFilterData;
	
	FOnCreateFilter OnCreateFilter;
	
	FOnDeleteFilter OnDeleteFilter;
	
	FOnCancelClicked OnCancelClicked;

	FOnModifyFilter OnModifyFilter;

	FOnGetFilterLabels OnGetFilterLabels;

	/* The color block widget that edits the filter color */
	TSharedPtr<SColorBlock> ColorBlock;

	/* The widget that edits the filter label */
	TSharedPtr<SEditableTextBox> FilterLabelTextBox;
};

#undef UE_API
