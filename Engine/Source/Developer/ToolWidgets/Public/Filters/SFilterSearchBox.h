// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Containers/UnrealString.h"
#include "Framework/SlateDelegates.h"

#define UE_API TOOLWIDGETS_API

class SMenuAnchor;
class SSearchBox;
template <typename T> class SListView;
class SFilterSearchBoxImpl;
class FTextSelection;
struct FTextLocation;

/** A SearchBox widget that contains support for displaying search history
 * When used with a FilterBar widget (@see SBasicFilterBar or SFilterBar), it allows the user to save searches from the
 * history as filter pills
 */
class SFilterSearchBox : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnSaveSearchClicked, const FText&)

	SLATE_BEGIN_ARGS(SFilterSearchBox)
	: _OnTextChanged()
	, _OnTextCommitted()
	, _InitialText()
	, _HintText()
	, _ShowSearchHistory(true)
	, _MaxSearchHistory(5)
	, _DelayChangeNotificationsWhileTyping( false )
	, _DelayChangeNotificationsWhileTypingSeconds(0.25f)
	{}

	/** Invoked whenever the text changes */
	SLATE_EVENT( FOnTextChanged, OnTextChanged )

	/** Invoked whenever the text is committed (e.g. user presses enter) */
	SLATE_EVENT( FOnTextCommitted, OnTextCommitted )

	/** Initial text to display for the search text */
	SLATE_ATTRIBUTE( FText, InitialText )

	/** Hint text to display for the search text when there is no value */
	SLATE_ATTRIBUTE( FText, HintText )

	/** Whether we should show a dropdown containing the last few searches */
	SLATE_ATTRIBUTE(bool, ShowSearchHistory)

	/** The maximum number of items to show in the Search History */
	SLATE_ARGUMENT(int32, MaxSearchHistory)

	/** The Search History to restore */
	SLATE_ARGUMENT(TArrayView<FText>, SearchHistory)

	/** Whether the SearchBox should delay notifying listeners of text changed events until the user is done typing */
	SLATE_ATTRIBUTE( bool, DelayChangeNotificationsWhileTyping )
	
	/** If we're delaying change notifications, how many seconds should we wait? */
    SLATE_ATTRIBUTE( float, DelayChangeNotificationsWhileTypingSeconds )

	/** Callback delegate to have first chance handling of the OnKeyDown event */
	SLATE_EVENT(FOnKeyDown, OnKeyDownHandler)

	/** Handler for when the + Button next to a search is clicked */
	SLATE_EVENT(FOnSaveSearchClicked, OnSaveSearchClicked)

	/**
	 * Optional widget to display on the right side of the search box, after the existing buttons.
	 * EXPERIMENTAL: This API is subject to change.
	 */
	PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	SLATE_ARGUMENT(TSharedPtr<SWidget>, ExtraWidget)
	PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct( const FArguments& InArgs );

	/** Sets the text string currently being edited */
	UE_API void SetText(const TAttribute< FText >& InNewText);

	/** Get the text string currently being edited */
	UE_API FText GetText() const;

	/** Get the current Search History excluding the placeholder empty search text */
	UE_API TArray<FText> GetSearchHistory() const;

	/** Set or clear the current error reporting information for this search box */
	UE_API void SetError( const FText& InError );
	UE_API void SetError( const FString& InError );

	/** SWidget interface */
	UE_API void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	UE_API virtual bool HasKeyboardFocus() const override;
	UE_API bool SupportsKeyboardFocus() const;
	UE_API FReply OnFocusReceived( const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent );

	/** Show a + button next to the current search, and set the handler for when that is clicked */
	UE_API void SetOnSaveSearchHandler(FOnSaveSearchClicked InOnSaveSearchHandler);

	/** Get the current selection */
	UE_API FTextSelection GetSelection() const;

	/** Select a block of text */
	UE_API void SelectText(const FTextLocation& InSelectionStart, const FTextLocation& InCursorLocation);

private:
	
	/** Handler for when text in the editable text box changed */
	void HandleTextChanged(const FText& NewText);

	/** Handler for when text in the editable text box changed */
	void HandleTextCommitted(const FText& NewText, ETextCommit::Type CommitType);

	/** Add the given search text to the search history */
	void UpdateSearchHistory(const FText &NewText);

	/** Called by SListView when the selection changes in the search history list */
	void OnSelectionChanged( TSharedPtr<FText> NewValue, ESelectInfo::Type SelectInfo );

	/** Makes the widget for a single search history row in the list view */
	TSharedRef<ITableRow> MakeSearchHistoryRowWidget(TSharedPtr<FText> SearchText, const TSharedRef<STableViewBase>& OwnerTable);

	/** Handler for when the chevron to open the search history is clicked */
	FReply OnClickedSearchHistory();

	/** Whether the search history dropdown chevron is visible */
	EVisibility GetSearchHistoryVisibility() const;

private:

	/* Whether we are showing the search history */
	TAttribute<bool> bShowSearchHistory;
	
	/** The actual Search Box */
	TSharedPtr<SFilterSearchBoxImpl> SearchBox;
	
	/** Delegate for when text is changed in the edit box */
	FOnTextChanged OnTextChanged;

	/** Delegate for when text is changed in the edit box */
	FOnTextCommitted OnTextCommitted;

	/** The max amount of items to show in the Search History */
	int32 MaxSearchHistory;

	/** The array containing the last few searched items */
	TArray<TSharedPtr<FText>> SearchHistory;

	/** Search History elements */
	TSharedPtr< SMenuAnchor > SearchHistoryBox;

	/** The ListView containing our search history */
	TSharedPtr<SListView< TSharedPtr<FText>>> SearchHistoryListView;

	/** The Text that shows up when there are no items in the search history */
	TSharedPtr<FText> EmptySearchHistoryText;
};

#undef UE_API
