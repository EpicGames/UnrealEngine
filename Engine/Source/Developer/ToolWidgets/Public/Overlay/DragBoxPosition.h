// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Types/SlateEnums.h"
#include "DragBoxPosition.generated.h"

/** Used to restore position of widget in SDraggableBoxOverlay. */
USTRUCT()
struct FToolWidget_DragBoxPosition
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Layout")
	FVector2f RelativeOffset = FVector2f::ZeroVector;

	UPROPERTY(EditAnywhere, Category = "Layout")
	TEnumAsByte<EHorizontalAlignment> HAlign = HAlign_Left;

	UPROPERTY(EditAnywhere, Category = "Layout")
	TEnumAsByte<EVerticalAlignment> VAlign = VAlign_Bottom;

	FToolWidget_DragBoxPosition() = default;
	explicit FToolWidget_DragBoxPosition(
		const FVector2f& RelativeOffset, const TEnumAsByte<EHorizontalAlignment>& HAlign, const TEnumAsByte<EVerticalAlignment>& VAlign
		)
		: RelativeOffset(RelativeOffset)
		, HAlign(HAlign)
		, VAlign(VAlign)
	{}
	
	bool operator==(const FToolWidget_DragBoxPosition& Other) const
	{
		return Other.RelativeOffset == RelativeOffset && HAlign == Other.HAlign && VAlign == Other.VAlign;
	}
};
