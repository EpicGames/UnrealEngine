// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Layout/Margin.h"
#include "Types/SlateEnums.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API TOOLWIDGETS_API

class SBox;
struct FToolWidget_DragBoxPosition;

namespace UE::ToolWidgets
{

/** Flags that specify which edges of an SDraggableBoxOverlay are user-resizable. */
enum class EResizeEdges : uint8
{
	None   = 0,
	Left   = 1 << 0,
	Right  = 1 << 1,
	Top    = 1 << 2,
	Bottom = 1 << 3,
	All    = Left | Right | Top | Bottom,
};
ENUM_CLASS_FLAGS(EResizeEdges)

/**
 * Class that can be used to place a draggable box into a viewport or some other large widget as an overlay.
 * Just place the widget that you want to be draggable as the contents.
 * 
 * This overlay will remain confined to the bounds of the viewport dimensions.
 * It anchors the content to the corners of the parent widget: e.g. when a user drags it from the bottom-left to the bottom-right corner and
 * drags the right side of the viewport, the widget moves relative to the right side.
 */
class SDraggableBoxOverlay : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDraggableBoxOverlay)
		: _HAlign(HAlign_Left)
		, _VAlign(VAlign_Bottom)
		, _InitialAlignmentOffset(FVector2f::ZeroVector)
		, _Draggable(true)
		, _Resizable(EResizeEdges::None)
		, _ResizeHandleThickness(5.f)
		, _MinimumBoxWidth(TOptional<float>(20.f))
		, _MinimumBoxHeight(TOptional<float>(20.f))
	{}
	/** Use this to specify whether to align to the left or right. Center and Fill prevent repositioning horizontally. */
	SLATE_ARGUMENT(EHorizontalAlignment, HAlign)
	/** Use this to specify whether to align to the top or bottom. Center and Fill prevent repositioning vertically. */
	SLATE_ARGUMENT(EVerticalAlignment, VAlign)
	
	/** The initial offset this widget should have relative to HAlign and VAlign. */
	SLATE_ARGUMENT(FVector2f, InitialAlignmentOffset)
	/** Whether this widget should be allowed to be dragged. */
	SLATE_ATTRIBUTE_DEPRECATED(bool, IsDraggable, 5.8, "Use Draggable instead.")
	/** Whether this widget should be allowed to be dragged. */
	SLATE_ATTRIBUTE(bool, Draggable)
	/** Whether this widget can override center alignment when dragged. */
	SLATE_ATTRIBUTE(bool, DragOverridesCenterAlignment)

	/**
	 * Invoked when the user has finished dragging the widget to a new position.
	 * This is useful if you want to save the box position persistently: call GetDragBoxPosition and save it in a config file.
	 */
	SLATE_EVENT(FSimpleDelegate, OnUserDraggedToNewPosition)

	/** Which edges the user is allowed to resize by dragging. Default: None (resizing disabled). */
	SLATE_ARGUMENT(EResizeEdges, Resizable)
	/** Thickness (in local pixels) of the invisible resize hit-target at each enabled edge. */
	SLATE_ARGUMENT(float, ResizeHandleThickness)
	/** Minimum allowed box width when the user resizes. */
	SLATE_ATTRIBUTE(TOptional<float>, MinimumBoxWidth)
	/** Minimum allowed box height when the user resizes. */
	SLATE_ATTRIBUTE(TOptional<float>, MinimumBoxHeight)
	/** Maximum allowed box width when the user resizes. */
	SLATE_ATTRIBUTE(TOptional<float>, MaximumBoxWidth)
	/** Maximum allowed box height when the user resizes. */
	SLATE_ATTRIBUTE(TOptional<float>, MaximumBoxHeight)
	/** Explicit width override. Unset (default) means the box sizes itself to its content width. */
	SLATE_ARGUMENT(TOptional<float>, WidthOverride)
	/** Explicit height override. Unset (default) means the box sizes itself to its content height. */
	SLATE_ARGUMENT(TOptional<float>, HeightOverride)
	/** Invoked when the user finishes a resize operation. */
	SLATE_EVENT(FSimpleDelegate, OnUserResized)

	/** The content you want to be able to drag around in the parent widget. */
	SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	/** @return The current position of the box on the viewport relative to HAlign and VAlign. */
	UE_API FVector2f GetBoxAlignmentOffset() const;

	/*
	 * Sets the content's position relative to the current anchor points.
	 * 
	 * @param InOffset Offset relative to the current HAlign and VAlign, e.g. X = 100 means an offset of 100 from the left or right widget side.
	 * @param bInRecomputeAnchorPoints Whether you want HAlign and VAlign recomputed,
	 *	e.g. if HAlign is currently left, parent is 200 wide, and InOffset.X == 150, this would recompute HAlign to be right.
	 */
	UE_API void SetBoxAlignmentOffset(const FVector2f& InOffset, bool bInRecomputeAnchorPoints = true);

	UE_API EHorizontalAlignment GetBoxHorizontalAlignment() const;
	UE_API void SetBoxHorizontalAlignment(EHorizontalAlignment InAlignment);

	UE_API EVerticalAlignment GetBoxVerticalAlignment() const;
	UE_API void SetBoxVerticalAlignment(EVerticalAlignment InAlignment);

	/** Gets information to restore this content's position. This is useful for saving in config files. */
	UE_API FToolWidget_DragBoxPosition GetDragBoxPosition() const;
	/** Sets the content's position. This is useful for loading a widget position saved in a config file.*/
	UE_API void SetDragBoxPosition(const FToolWidget_DragBoxPosition& InPosition);
	/** Restores the content position. This is useful for loading a widget position saved in a config file. */
	UE_DEPRECATED(5.8, "Use SetDragBoxPosition instead.")
	UE_API void RestoreFromDragBoxPosition(const FToolWidget_DragBoxPosition& InWidgetPosition);

	/** Returns the explicit width override set by a user resize, or unset if the box is content-sized on this axis. */
	UE_API TOptional<float> GetWidthOverride() const;
	/** Sets an explicit width override. Pass an unset optional to revert to content sizing on this axis. */
	UE_API void SetWidthOverride(const TOptional<float>& NewWidth);

	/** Returns the explicit height override set by a user resize, or unset if the box is content-sized on this axis. */
	UE_API TOptional<float> GetHeightOverride() const;
	/** Sets an explicit height override. Pass an unset optional to revert to content sizing on this axis. */
	UE_API void SetHeightOverride(const TOptional<float>& NewHeight);
	
	UE_API TOptional<float> GetMinimumBoxWidth() const;
	UE_API TOptional<float> GetMaximumBoxWidth() const;
	UE_API TOptional<float> GetMinimumBoxHeight() const;
	UE_API TOptional<float> GetMaximumBoxHeight() const;

protected:

	/** Contains the DraggableBox and repositions it relative to our parent. */
	TSharedPtr<SBox> Container;
	/** Contains the content and handles the drag-drop behaviour. */
	TSharedPtr<SWidget> DraggableBox;

	/** Width override set by a user resize operation. Unset means the box sizes itself to its content on this axis. */
	TOptional<float> WidthOverride;
	/** Height override set by a user resize operation. Unset means the box sizes itself to its content on this axis. */
	TOptional<float> HeightOverride;
	/** Minimum allowed box width when the user resizes. */
	TAttribute<TOptional<float>> MinimumBoxWidth;
	/** Minimum allowed box height when the user resizes. */
	TAttribute<TOptional<float>> MinimumBoxHeight;
	/** Maximum allowed box width when the user resizes. */
	TAttribute<TOptional<float>> MaximumBoxWidth;
	/** Maximum allowed box height when the user resizes. */
	TAttribute<TOptional<float>> MaximumBoxHeight;

	/** Affects the horizontal alignment of the content widget relative to the parent widget. */
	EHorizontalAlignment HorizontalAlignment = HAlign_Fill;
	/** Affects the vertical alignment of the content widget relative to the parent widget. */
	EVerticalAlignment VerticalAlignment = VAlign_Fill;
	/** The padding of the content widget. */
	FMargin Padding;

	UE_API FMargin GetPadding() const;

	/**
	 * Recomputes HorizontalAlignment and VerticalAlignment if InOffset is too far away from the current corner.
	 * @param InOffset Offset relative to current HorizontalAlignment and VerticalAlignment
	 * @return New offset relative to recomputed HorizontalAlignment and VerticalAlignment
	 */
	UE_API FVector2f RecomputeAnchorPoints(const FVector2f& InOffset);
};
}

#undef UE_API
