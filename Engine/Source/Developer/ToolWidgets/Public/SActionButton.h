// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "ToolWidgetsSlateTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboButton.h"

#define UE_API TOOLWIDGETS_API

class IActionButton
{
public:
	virtual ~IActionButton() = default;

	virtual void SetMenuContentWidgetToFocus(TWeakPtr<SWidget> InWidget) = 0;
	virtual void SetIsMenuOpen(bool bInIsOpen, bool bInIsFocused) = 0;
};

/** A Button that is used to call out/highlight an option. It can also be used to open a menu. */
class SActionButton
	: public SCompoundWidget
	, public IActionButton
{
public:
	SLATE_BEGIN_ARGS(SActionButton)
		: _ActionButtonStyle(nullptr)
		, _ButtonStyle(nullptr)
		, _IconButtonStyle(nullptr)
		, _ComboButtonStyle(nullptr)
		, _TextBlockStyle(nullptr)
		{}

		SLATE_STYLE_ARGUMENT(FActionButtonStyle, ActionButtonStyle)

		/** Used to describe the intent of this button. */
		SLATE_ATTRIBUTE(EActionButtonType, ActionButtonType)

		/** Optionally specify the button style, which may override visual properties determined by ActionButtonStyle. */
		SLATE_STYLE_ARGUMENT(FButtonStyle, ButtonStyle)

		/** Optionally specify the button style when an icon is present, which may override visual properties determined by ActionButtonStyle. */
		SLATE_STYLE_ARGUMENT(FButtonStyle, IconButtonStyle)

		/** Spacing between button's border and the content. */
		SLATE_ATTRIBUTE(FMargin, ButtonContentPadding)

		/** Optionally specify the combo button style, which may override visual properties determined by ActionButtonStyle. */
		SLATE_STYLE_ARGUMENT(FComboButtonStyle, ComboButtonStyle)

		/** Whether to show a down arrow for the combo button. Default is determined by the style. */
		SLATE_ARGUMENT(TOptional<bool>, HasDownArrow)

		/** Horizontal Content alignment within the button. Default is determined by the style. */
		SLATE_ARGUMENT(TOptional<EHorizontalAlignment>, HorizontalContentAlignment)

		/** The text to display in the button. */
		SLATE_ATTRIBUTE(FText, Text)

		/** The style of the text block, which dictates the font, color, and shadow options. */
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, TextBlockStyle)

		/** Optionally specify the Icon to display in the button. */
		SLATE_ATTRIBUTE(const FSlateBrush*, Icon)

		/** Icon Color/Tint, defaults to White. */
		SLATE_ATTRIBUTE(FSlateColor, IconColorAndOpacity)

		#pragma region Button
		/** The clicked handler. Note that if this is set, the button will behave as though it were just a button.
		 * This means that OnGetMenuContent, OnComboBoxOpened and OnMenuOpenChanged will all be ignored, since there is no menu.
		 */
		SLATE_EVENT(FOnClicked, OnClicked)
		#pragma endregion Button

		#pragma region ComboButton
		/** The static menu content widget. */
		SLATE_NAMED_SLOT(FArguments, MenuContent)

		SLATE_EVENT(FOnGetContent, OnGetMenuContent)
		SLATE_EVENT(FOnComboBoxOpened, OnComboBoxOpened)
		SLATE_EVENT(FOnIsOpenChanged, OnMenuOpenChanged)
		#pragma endregion ComboButton
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	//~ Begin IActionButton
	UE_API virtual void SetMenuContentWidgetToFocus(TWeakPtr<SWidget> InWidget) override;
	UE_API virtual void SetIsMenuOpen(bool bInIsOpen, bool bInIsFocused) override;
	//~ End IActionButton

protected:
	UE_API virtual FSlateColor GetIconColorAndOpacity() const;

protected:
	TSharedPtr<SComboButton> ComboButton;
	TSharedPtr<class SButton> Button;

	TAttribute<EActionButtonType> ActionButtonType = EActionButtonType::Default;
	const FActionButtonStyle* ActionButtonStyle = nullptr;
	const FButtonStyle* ButtonStyle = nullptr;
	const FButtonStyle* IconButtonStyle = nullptr;
	const FComboButtonStyle* ComboButtonStyle = nullptr;
	const FTextBlockStyle* TextBlockStyle = nullptr;
};

#undef UE_API
