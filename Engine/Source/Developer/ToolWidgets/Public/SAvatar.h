// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API TOOLWIDGETS_API

/** Widget for displaying a user avatar icon.
 * The avatar's background color is computed based on the Identifier.
 * The avatar's displayed letter is the initial letter of the Description if it's alpha numeric, or a question mark otherwise.
 */
class SAvatar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAvatar) :
		_HeightOverride(17.f),
		_WidthOverride(17.f),
		_ShowInitial(true)
	{}
		SLATE_ARGUMENT(FString, Identifier)
		SLATE_ARGUMENT(FString, Description)
		SLATE_ARGUMENT(float, HeightOverride)
		SLATE_ARGUMENT(float, WidthOverride)
		SLATE_ARGUMENT(bool, ShowInitial)
	SLATE_END_ARGS()

public:
	/** Construct this widget */
	UE_API void Construct(const FArguments& InArgs);

	/** The custom OnPaint handler necessary because we want to draw a circle */
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

private:
	/** Computes a hash value using the djb2 algorithm.
	 * @return the hash value of the Identifier
	 */
	uint32 Hash() const;
	
	/** Computes the avatar's background color based on the Identifier
	 * Calls the Hash() function above to calculate an HSV color which is
	 * transformed into an FColor that ultimately represents the user's avatar color.
	 * 
	 * @return the RGB representation of the Identifier
	 */
	FColor ComputeBackgroundColor() const;

	/** The Identifier of the avatar */
	FString Identifier;

	/** The Description of the avatar */
	FString Description;

	/** The BackgroundColor used */
	FColor BackgroundColor;

	/** The ForegroundColor used (text) */
	FColor ForegroundColor;

	/** The setting that determines if we're showing the initial */
	bool bShowInitial;
};

#undef UE_API
