// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API TOOLWIDGETS_API

enum class ECheckBoxState : uint8;
class ITableRow;
class SCheckBox;
class STableViewBase;

template<typename T>
class SListView;

namespace CheckBoxList
{
	struct FItemPair;
}

DECLARE_DELEGATE_OneParam( FOnCheckListItemStateChanged, int );

/** A widget that can be used inside a CustomDialog to display a list of checkboxes */
class SCheckBoxList: public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SCheckBoxList)
		: _CheckBoxStyle(&FCoreStyle::Get().GetWidgetStyle<FCheckBoxStyle>("Checkbox"))
		, _IncludeGlobalCheckBoxInHeaderRow(true)
		{}
		/** The styling of the CheckBox */
		SLATE_STYLE_ARGUMENT(FCheckBoxStyle, CheckBoxStyle)
		/** The label of the item column header */
		SLATE_ARGUMENT(FText, ItemHeaderLabel)
		/** Optionally display a checkbox by the column header that toggles all items */
		SLATE_ARGUMENT(bool, IncludeGlobalCheckBoxInHeaderRow)
		/** Callback when any checkbox is changed. Parameter is the index of the item, or -1 if it was the "All"/Global checkbox */
		SLATE_EVENT( FOnCheckListItemStateChanged, OnItemCheckStateChanged )
	SLATE_END_ARGS()

public:
	UE_API void Construct(const FArguments& Arguments);
	UE_API void Construct(const FArguments& Arguments, const TArray<FText>& Items, bool bIsChecked);
	UE_API void Construct(const FArguments& Arguments, const TArray<TSharedRef<SWidget>>& Items, bool bIsChecked);

	UE_API int32 AddItem(const FText& Text, bool bIsChecked);
	UE_API int32 AddItem(TSharedRef<SWidget> Widget, bool bIsChecked);
	UE_API void RemoveAll();
	UE_API void RemoveItem(int32 Index);
	UE_API bool IsItemChecked(int32 Index) const;

	UE_API TArray<bool> GetValues() const;
	UE_API int32 GetNumCheckboxes() const;

private:
	void UpdateAllChecked();
	ECheckBoxState GetToggleSelectedState() const;
	void OnToggleSelectedCheckBox(ECheckBoxState InNewState);
	void OnItemCheckBox(TSharedRef<CheckBoxList::FItemPair> InItem);

	TSharedRef<ITableRow> HandleGenerateRow(TSharedRef<CheckBoxList::FItemPair> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	ECheckBoxState bAllCheckedState = ECheckBoxState::Unchecked;
	TArray<TSharedRef<CheckBoxList::FItemPair>> Items;

	const FCheckBoxStyle* CheckBoxStyle = nullptr;
	TSharedPtr<SListView<TSharedRef<CheckBoxList::FItemPair>>> ListView;

	FOnCheckListItemStateChanged OnItemCheckStateChanged;
};

#undef UE_API
