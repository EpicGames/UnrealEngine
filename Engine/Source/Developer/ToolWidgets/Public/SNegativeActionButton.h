// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "SActionButton.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API TOOLWIDGETS_API

enum class EActionButtonStyle
{
	Warning,
	Error
};

/** A Button that is used to call out/highlight a negative option (Warnings or Errors like Force Delete).
*   It can also be used to open a menu.
*/
class SNegativeActionButton
	: public SCompoundWidget
	, public IActionButton
{
public:
	SLATE_BEGIN_ARGS(SNegativeActionButton)
		: _ActionButtonStyle(EActionButtonStyle::Error)
		{}

		/** Determines whether this is a Warning or Error. */
		SLATE_ATTRIBUTE(EActionButtonStyle, ActionButtonStyle)

		/** The text to display in the button. */
		SLATE_ATTRIBUTE(FText, Text)

		/** Optionally specify the Icon to display in the button. */
		SLATE_ATTRIBUTE(const FSlateBrush*, Icon)

		/** The clicked handler. Note that if this is set, the button will behave as though it were just a button.
		 * This means that OnGetMenuContent, OnComboBoxOpened and OnMenuOpenChanged will all be ignored, since there is no menu.
		 */
		SLATE_EVENT(FOnClicked, OnClicked)

		/** The static menu content widget. */
		SLATE_NAMED_SLOT(FArguments, MenuContent)

		SLATE_EVENT(FOnGetContent, OnGetMenuContent)
		SLATE_EVENT(FOnComboBoxOpened, OnComboBoxOpened)
		SLATE_EVENT(FOnIsOpenChanged, OnMenuOpenChanged)
	SLATE_END_ARGS()

	SNegativeActionButton() = default;

	UE_API void Construct(const FArguments& InArgs);

	//~ Begin IActionButton
	UE_API virtual void SetMenuContentWidgetToFocus(TWeakPtr<SWidget> InWidget) override;
	UE_API virtual void SetIsMenuOpen(bool bInIsOpen, bool bInIsFocused) override;
	//~ End IActionButton

private:
	TSharedPtr<SActionButton> ActionButton;
};

#undef UE_API
