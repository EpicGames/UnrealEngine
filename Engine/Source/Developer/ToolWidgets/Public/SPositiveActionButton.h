// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "SActionButton.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboButton.h"

#define UE_API TOOLWIDGETS_API

/** A Button that is used to call out/highlight a positive option (Add, Save etc). It can also be used to open a menu.
*/
class SPositiveActionButton
	: public SCompoundWidget
	, public IActionButton
{
public:
	SLATE_BEGIN_ARGS(SPositiveActionButton)
		: _Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
		{}

		/** The text to display in the button. */
		SLATE_ATTRIBUTE(FText, Text)

		SLATE_ATTRIBUTE(const FSlateBrush*, Icon)

		/** The clicked handler. Note that if this is set, the button will behave as though it were just a button.
		 * This means that OnGetMenuContent, OnComboBoxOpened and OnMenuOpenChanged will all be ignored, since there is no menu.
		 */
		SLATE_EVENT(FOnClicked, OnClicked)

		/** The static menu content widget. */
		SLATE_NAMED_SLOT(FArguments, MenuContent)

		SLATE_EVENT(FOnGetContent, OnGetMenuContent)
		SLATE_EVENT(FOnComboBoxOpened, OnComboBoxOpened)
		SLATE_EVENT(FOnIsOpenChanged, OnMenuOpenChanged)
	SLATE_END_ARGS()

	SPositiveActionButton() = default;

	UE_API void Construct(const FArguments& InArgs);

	//~ Begin IActionButton
	UE_API virtual void SetMenuContentWidgetToFocus(TWeakPtr<SWidget> InWidget) override;
	UE_API virtual void SetIsMenuOpen(bool bInIsOpen, bool bInIsFocused) override;
	//~ End IActionButton

private:
	TSharedPtr<SActionButton> ActionButton;
};

#undef UE_API
