// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"

#define UE_API TOOLWIDGETS_API

template <>
struct TWidgetTypeTraits<class SPrimaryButton>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SPrimaryButton
	: public SButton
{
public:
	SLATE_BEGIN_ARGS(SPrimaryButton)
	{}
		/** The text to display in the button. */
		SLATE_ATTRIBUTE(FText, Text)

		/** Optional icon to display in the button. */
		SLATE_ATTRIBUTE(const FSlateBrush*, Icon)

		/** 
		 * Called when the button is clicked  
		 */
		SLATE_EVENT(FOnClicked, OnClicked)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);
};

#undef UE_API
