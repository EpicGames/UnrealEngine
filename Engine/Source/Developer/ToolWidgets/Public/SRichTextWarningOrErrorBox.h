// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Layout/SBorder.h"
#include "SWarningOrErrorBox.h"

#define UE_API TOOLWIDGETS_API

class SRichTextWarningOrErrorBox : public SWarningOrErrorBox
{
public:

	UE_API void Construct(const FArguments& InArgs);

private:
	TAttribute<EMessageStyle> MessageStyle;
};

#undef UE_API
