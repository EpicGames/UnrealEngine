// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Delegates/Delegate.h"
#include "Layout/Visibility.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API TOOLWIDGETS_API

class SSearchBox;

DECLARE_DELEGATE(FOnSearchBoxShown)

template<>
struct TWidgetTypeTraits<class SSearchToggleButton>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

class SSearchToggleButton : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSearchToggleButton)
		: _Style(&FAppStyle::Get().GetWidgetStyle<FSearchBoxStyle>("SearchBox"))
	{}
		/** Search box style (used to match the glass icon) */
		SLATE_STYLE_ARGUMENT(FSearchBoxStyle, Style)

		/** Event fired when the associated search box is made visible */
		SLATE_EVENT(FOnSearchBoxShown, OnSearchBoxShown)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TSharedRef<SSearchBox> SearchBox);

	/** @return true if the search area is expanded and the search box exposed */
	bool IsExpanded() const { return bIsExpanded; }

	/** Sets whether or not the search area is expanded to expose the search box */
	UE_API void SetExpanded(bool bInExpanded);
private:

	UE_API ECheckBoxState GetToggleButtonState() const;
	UE_API void OnToggleButtonStateChanged(ECheckBoxState CheckBoxState);
	UE_API EVisibility GetSearchBoxVisibility() const;
private:
	const FSearchBoxStyle* SearchStyle;
	bool bIsExpanded;
	FOnSearchBoxShown OnSearchBoxShown;
	TWeakPtr<SSearchBox> SearchBoxPtr;
};

#undef UE_API
