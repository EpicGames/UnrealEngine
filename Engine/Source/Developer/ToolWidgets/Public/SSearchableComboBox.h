// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.8, "This header has moved to the Slate module - Widgets/Input/SSearchableComboBox.h.")

// Include the new file so that the project still compiles, but has warnings
#include "Widgets/Input/SSearchableComboBox.h"
