// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Styling/CoreStyle.h"
#include "Widgets/SCompoundWidget.h"
#include "Animation/CurveSequence.h"

#define UE_API TOOLWIDGETS_API

struct FSegmentedProgressBarStyle
{
	FSlateColor BusyColor;
	FSlateColor PendingColor;
	FSlateColor ErrorColor;
	FSlateColor CompleteColor;
	FSlateColor CanceledColor;
	FSlateColor SkippedColor;
	FSlateColor AllCompleteColor;

	const FSlateBrush* FullCircleBrush = nullptr;
	const FSlateBrush* OuterCircleBrush = nullptr;
	const FSlateBrush* OuterCircleBusyBrush = nullptr;
	const FSlateBrush* BadgeSuccessBrush = nullptr;
	const FSlateBrush* BadgeErrorBrush = nullptr;
	const FSlateBrush* BadgeAllCompleteBrush = nullptr;

	FSegmentedProgressBarStyle& SetBusyColor(FSlateColor InColor) { BusyColor = InColor; return *this; }
	FSegmentedProgressBarStyle& SetPendingColor(FSlateColor InColor) { PendingColor = InColor; return *this; }
	FSegmentedProgressBarStyle& SetErrorColor(FSlateColor InColor) { ErrorColor = InColor; return *this; }
	FSegmentedProgressBarStyle& SetCompleteColor(FSlateColor InColor) { CompleteColor = InColor; return *this; }
	FSegmentedProgressBarStyle& SetCanceledColor(FSlateColor InColor) { CanceledColor = InColor; return *this; }
	FSegmentedProgressBarStyle& SetSkippedColor(FSlateColor InColor) { SkippedColor = InColor; return *this; }
	FSegmentedProgressBarStyle& SetAllCompleteColor(FSlateColor InColor) { AllCompleteColor = InColor; return *this; }

	FSegmentedProgressBarStyle& SetFullCircleBrush(const FSlateBrush* InBrush) { FullCircleBrush = InBrush; return *this; }
	FSegmentedProgressBarStyle& SetOuterCircleBrush(const FSlateBrush* InBrush) { OuterCircleBrush = InBrush; return *this; }
	FSegmentedProgressBarStyle& SetOuterCircleBusyBrush(const FSlateBrush* InBrush) { OuterCircleBusyBrush = InBrush; return *this; }
	FSegmentedProgressBarStyle& SetBadgeSuccessBrush(const FSlateBrush* InBrush) { BadgeSuccessBrush = InBrush; return *this; }
	FSegmentedProgressBarStyle& SetBadgeErrorBrush(const FSlateBrush* InBrush) { BadgeErrorBrush = InBrush; return *this; }
	FSegmentedProgressBarStyle& SetBadgeAllCompleteBrush(const FSlateBrush* InBrush) { BadgeAllCompleteBrush = InBrush; return *this; }
};

class SSegmentedProgressBar : public SCompoundWidget
{
public:
	enum class EState : uint8
	{
		None,
		Busy,
		Canceled,
		Completed,
		Failed,
		Pending,
		Skipped,
	};


	struct FSlot : public TSlotBase<FSlot>
	{
	public:
		FSlot()
			: TSlotBase<FSlot>()
			, Image(nullptr)
		{}

		SLATE_SLOT_BEGIN_ARGS(FSlot, TSlotBase<FSlot>)
			SLATE_ATTRIBUTE(const FSlateBrush*, Image)
			SLATE_ATTRIBUTE(EState, State)
			SLATE_ATTRIBUTE(FText, ToolTipText)
			SLATE_ARGUMENT(TSharedPtr<SWidget>, LabelContent)
		SLATE_SLOT_END_ARGS()

		void Construct(const FChildren& SlotOwner, FSlotArguments&& InArgs)
		{
			TSlotBase<FSlot>::Construct(SlotOwner, MoveTemp(InArgs));
			if (InArgs._Image.IsSet())
			{
				Image = MoveTemp(InArgs._Image);
			}
			if (InArgs._State.IsSet())
			{
				State = MoveTemp(InArgs._State);
			}
			if (InArgs._ToolTipText.IsSet())
			{
				ToolTipText = MoveTemp(InArgs._ToolTipText);
			}
			if (InArgs._LabelContent)
			{
				LabelContent = InArgs._LabelContent;
			}
		}

	protected:
		TAttribute<const FSlateBrush*> Image;
		TAttribute<EState> State;
		TAttribute<FText> ToolTipText;
		TSharedPtr<SWidget> LabelContent;

		friend SSegmentedProgressBar;
	};

	static FSlot::FSlotArguments Slot()
	{
		return FSlot::FSlotArguments(MakeUnique<FSlot>());
	}

	SLATE_BEGIN_ARGS(SSegmentedProgressBar){}
		SLATE_SLOT_ARGUMENT(FSlot, Slots)
		SLATE_ARGUMENT(FSegmentedProgressBarStyle, Style)
		SLATE_ARGUMENT_DEFAULT(bool, bAutoPromoteNextPending) = true;
	SLATE_END_ARGS()

	using FScopedWidgetSlotArguments = typename TPanelChildren<FSlot>::FScopedWidgetSlotArguments;
	FScopedWidgetSlotArguments AddSlot(bool bRebuildChildren = true)
	{
		return InsertSlot(INDEX_NONE, bRebuildChildren);
	}

	UE_API FScopedWidgetSlotArguments InsertSlot(int32 Index = INDEX_NONE, bool bRebuildChildren = true);

	FSlot& GetSlot(int32 SlotIndex)
	{
		return Children[SlotIndex];
	}

	const FSlot& GetSlot(int32 SlotIndex) const
	{
		return Children[SlotIndex];
	}

	int32 NumSlots() const
	{
		return Children.Num();
	}

	void ClearChildren()
	{
		Children.Empty();
	}

	SSegmentedProgressBar()
		: Children(this)
	{
		bCanSupportFocus = false;
	}



	UE_API void Construct(const FArguments& Arguments);
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	void RebuildChildren();
	void AddChildSeparatorBar( TSharedPtr<class SHorizontalBox> SlotBox, int32 ChildIndex ) const;
	TSharedRef<SWidget> ConstructChild( const FSlot& Slot, int32 ChildIndex ) const;

	void CacheChildStates();
	EState GetChildState( int32 ChildIndex ) const;

	FLinearColor GetSeparatorBarColor( int32 ChildIndex ) const;
	FSlateColor GetCircleColor( int32 ChildIndex ) const;
	FSlateColor GetIconColor( int32 ChildIndex ) const;
	EVisibility GetFullCircleVisibility( int32 ChildIndex ) const;
	EVisibility GetOuterCircleVisibility( int32 ChildIndex ) const;
	EVisibility GetProgressCircleVisibility( int32 ChildIndex ) const;
	EVisibility GetOverlayVisibility( int32 ChildIndex ) const;
	const FSlateBrush* GetOverlayIcon( int32 ChildIndex ) const;
	TOptional<FSlateRenderTransform> GetProgressCircleTransform( int32 ChildIndex ) const;

	FSegmentedProgressBarStyle Style;
	TPanelChildren<FSlot> Children;
	TArray<EState> CachedChildStates;
	EState CachedOverallState = EState::None;
	bool bAutoPromoteNextPending = true;

	FCurveSequence ThrobberAnimation;
	const float LineSize = 4;
};

#undef UE_API
