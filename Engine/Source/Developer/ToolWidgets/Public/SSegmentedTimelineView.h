// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API TOOLWIDGETS_API

class FPaintArgs;
class FSlateWindowElementList;
class FTimeSliderController;


// a timeline view which renders a block for in the timeline where the object exists
class SSegmentedTimelineView : public SCompoundWidget
{
public:
	struct FSegmentData
	{
	public:
		TArray<TRange<double>> Segments;
		// This is an optional array of alternating colors. Every segment will use the next color in this array and will wrap around
		// if the number of colors provided is > than the number of segments.
		TOptional<TArray<FLinearColor>> AlternatingSegmentsColors;
	};

	SLATE_BEGIN_ARGS(SSegmentedTimelineView)
		: _ViewRange(TRange<double>(0.0,10.0))
		, _DesiredSize(FVector2f(100.f,20.f))
	{}
    	/** View time range */
    	SLATE_ATTRIBUTE(TRange<double>, ViewRange);
	
		/** Existence Time Range */
		SLATE_ATTRIBUTE(TSharedPtr<FSegmentData>, SegmentData);

		/** Desired widget size */
		SLATE_ATTRIBUTE(FVector2f, DesiredSize);
	
		/** Fill Color */
		SLATE_ATTRIBUTE(FLinearColor, FillColor);
     
	SLATE_END_ARGS()


	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	UE_API void Construct( const FArguments& InArgs );

protected:
	// SWidget interface
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	UE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	
	UE_API int32 PaintBlock(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const;
	
	TAttribute<TRange<double>> ViewRange;
	TAttribute<TSharedPtr<FSegmentData>> SegmentData;
	TAttribute<FVector2f> DesiredSize;
	TAttribute<FLinearColor> FillColor;
	
	const FSlateBrush* WhiteBrush;
};

#undef UE_API
