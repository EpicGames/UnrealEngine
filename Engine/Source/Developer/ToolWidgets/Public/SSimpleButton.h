// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API TOOLWIDGETS_API

class SSimpleButton
	: public SButton
{
public:
	SLATE_BEGIN_ARGS(SSimpleButton)
	{}
		/** The text to display in the button. */
		SLATE_ATTRIBUTE(FText, Text)

		SLATE_ATTRIBUTE(const FSlateBrush*, Icon)

		/** The clicked handler. */
		SLATE_EVENT(FOnClicked, OnClicked)
	SLATE_END_ARGS()

	SSimpleButton() = default;

	UE_API void Construct(const FArguments& InArgs);
};

#undef UE_API
