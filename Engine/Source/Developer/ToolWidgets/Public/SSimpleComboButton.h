// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "SActionButton.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API TOOLWIDGETS_API

class SSimpleComboButton
	: public SComboButton
	, public IActionButton
{
public:
	SLATE_BEGIN_ARGS(SSimpleComboButton)
		: _HasDownArrow(false)
		, _UsesSmallText(false)
	{}
		/** The text to display in the button. */
		SLATE_ATTRIBUTE(FText, Text)

		SLATE_ATTRIBUTE(const FSlateBrush*, Icon)
	
		/** The static menu content widget. */
		SLATE_NAMED_SLOT(FArguments, MenuContent)

		SLATE_EVENT(FOnGetContent, OnGetMenuContent)
		SLATE_EVENT(FOnComboBoxOpened, OnComboBoxOpened)
		SLATE_EVENT(FOnIsOpenChanged, OnMenuOpenChanged)
		SLATE_ARGUMENT(bool, HasDownArrow)
		SLATE_ARGUMENT(bool, UsesSmallText)
	SLATE_END_ARGS()

	SSimpleComboButton() = default;

	UE_API void Construct(const FArguments& InArgs);

	//~ Begin IActionButton
	UE_API virtual void SetMenuContentWidgetToFocus(TWeakPtr<SWidget> InWidget) override;
	UE_API virtual void SetIsMenuOpen(bool bInIsOpen, bool bInIsFocused) override;
	//~ End IActionButton
};

#undef UE_API
