// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SScrollBar.h"

#define UE_API TOOLWIDGETS_API

class FPaintArgs;
class FSlateWindowElementList;
class FTimeSliderController;

class SSimpleTimeSlider : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam( FOnRangeChanged, TRange<double> )
	DECLARE_DELEGATE_TwoParams( FOnScrubPositionChanged, double, bool )
	DECLARE_DELEGATE_RetVal_OneParam( FString, FOnFormatTickLabel, double /* InSeconds */ )

	SLATE_BEGIN_ARGS(SSimpleTimeSlider)
		: _MirrorLabels( false )
		, _ScrubPosition(0.0)
		, _ViewRange(TRange<double>(0.0,10.0))
		, _ClampRange(TRange<double>(0.0,10.0))
		, _AllowZoom (true)
		, _AllowPan (true)
		, _CursorSize(0.f)
		, _ClampRangeHighlightColor(FLinearColor(0.05f,0.05f,0.05f,1.0f))
		, _ClampRangeHighlightSize(1.0f)
		, _DesiredSize(FVector2f(100.f,22.f))
	{}
		/* If we should mirror the labels on the timeline */
		SLATE_ATTRIBUTE( bool, MirrorLabels )
	
		/** The scrub position */
    	SLATE_ATTRIBUTE(double, ScrubPosition);
    
    	/** View time range */
    	SLATE_ATTRIBUTE(TRange<double>, ViewRange);
    
    	/** Clamp time range */
    	SLATE_ATTRIBUTE(TRange<double>, ClampRange);
    
    	/** If we are allowed to zoom */
    	SLATE_ATTRIBUTE(bool, AllowZoom);
	
    	/** If we are allowed to pan */
    	SLATE_ATTRIBUTE(bool, AllowPan);
    
    	/** Cursor range for data like histogram graphs, etc. */
    	SLATE_ATTRIBUTE(float, CursorSize);
	
    	/** Color overlay for active range */
		SLATE_ATTRIBUTE(FLinearColor, ClampRangeHighlightColor);

		/** Size of clamp range overlay 0.0 for none 1.0 for the height of the slider */
		SLATE_ATTRIBUTE(float, ClampRangeHighlightSize);

		/* Widget desired size */
		SLATE_ARGUMENT(FVector2f, DesiredSize);
    
	
		/** Called when the scrub position changes */
		SLATE_EVENT(FOnScrubPositionChanged, OnScrubPositionChanged);
	
		/** Called right before the scrubber begins to move */
		SLATE_EVENT(FSimpleDelegate, OnBeginScrubberMovement);
	
		/** Called right after the scrubber handle is released by the user */
		SLATE_EVENT(FSimpleDelegate, OnEndScrubberMovement);
	
    	/** Called when the view range changes */
    	SLATE_EVENT(FOnRangeChanged, OnViewRangeChanged);

    	/** Optional callback to format tick labels. If unbound, default "%.2f" / "%.3f" formatting is used. */
    	SLATE_EVENT(FOnFormatTickLabel, OnFormatTickLabel);

    	/** Minimum pixels between each minor tick mark. Larger values produce sparser ticks and labels. If unset, falls back to the default in ScrubConstants. */
    	SLATE_ARGUMENT(TOptional<uint32>, MinPixelsPerDisplayTick);

	SLATE_END_ARGS()


	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	UE_API void Construct( const FArguments& InArgs );

	TRange<double> GetTimeRange() { return ViewRange.Get(); }
	UE_API void SetTimeRange(double MinValue, double MaxValue);
	UE_API void SetClampRange(double MinValue, double MaxValue);
	bool IsPanning() { return bPanning; }

	/** Utility struct for converting between scrub range space and local/absolute screen space */
	struct FScrubRangeToScreen
	{
		FVector2f WidgetSize;
	
		TRange<double> ViewInput;
		double ViewInputRange;
		float PixelsPerInput;
	
		FScrubRangeToScreen(TRange<double> InViewInput, const UE::Slate::FDeprecateVector2DParameter& InWidgetSize )
		{
			WidgetSize = InWidgetSize;
	
			ViewInput = InViewInput;
			ViewInputRange = ViewInput.Size<double>();
			PixelsPerInput = ViewInputRange > 0.0 ? static_cast<float>(static_cast<double>(WidgetSize.X) / ViewInputRange) : 0.f;
		}
	
		/** Local Widget Space -> Curve Input domain. */
		double LocalXToInput(const float ScreenX) const
		{
			const float LocalX = ScreenX;
			return static_cast<double>(LocalX/PixelsPerInput) + ViewInput.GetLowerBoundValue();
		}
	
		/** Curve Input domain -> local Widget Space */
		float InputToLocalX(const double Input) const
		{
			return static_cast<float>(Input - ViewInput.GetLowerBoundValue()) * PixelsPerInput;
		}
	};

protected:
	
	struct FDrawTickArgs
    {
    	/** Geometry of the area */
    	FGeometry AllottedGeometry;
    	/** Clipping rect of the area */
    	FSlateRect ClippingRect;
    	/** Color of each tick */
    	FLinearColor TickColor;
    	/** Offset in Y where to start the tick */
    	float TickOffset;
    	/** Height in of major ticks */
    	float MajorTickHeight;
    	/** Start layer for elements */
    	int32 StartLayer;
    	/** Draw effects to apply */
    	ESlateDrawEffect DrawEffects;
    	/** Whether or not to only draw major ticks */
    	bool bOnlyDrawMajorTicks;
    	/** Whether or not to mirror labels */
    	bool bMirrorLabels;
    };

	// SWidget interface
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	UE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	UE_API virtual FReply OnPreviewMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	UE_API virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	
	UE_API int32 OnPaintTimeSlider( bool bMirrorLabels, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const;
	
	/**
	 * Draws time tick marks
	 *
	 * @param OutDrawElements	List to add draw elements to
	 * @param RangeToScreen		Time range to screen space converter
	 * @param InArgs			Parameters for drawing the tick lines
	 */
	UE_API void DrawTicks( FSlateWindowElementList& OutDrawElements, const FScrubRangeToScreen& RangeToScreen, FDrawTickArgs& InArgs ) const;

	UE_API double GetTimeAtCursorPosition(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const;
	
	/**
	 * Call this method when the user's interaction has changed the scrub position
	 *
	 * @param NewValue				Value resulting from the user's interaction
	 * @param bIsScrubbing			True if done via scrubbing, false if just releasing scrubbing
	 */
	UE_API void CommitScrubPosition( double NewValue, bool bIsScrubbing );

	TAttribute<double> ScrubPosition;
	TAttribute<TRange<double>> ViewRange;
	TAttribute<TRange<double>> ClampRange;
	TAttribute<double> TimeSnapInterval;
	TAttribute<bool> AllowZoom;
	TAttribute<bool> AllowPan;
	TAttribute<float> CursorSize;
	TAttribute<FLinearColor> ClampRangeHighlightColor;
	TAttribute<float> ClampRangeHighlightSize;
	TAttribute<bool> MirrorLabels;
	
	FOnScrubPositionChanged OnScrubPositionChanged;
	/** Called right before the scrubber begins to move */
	FSimpleDelegate OnBeginScrubberMovement;
	/** Delegate called right after the scrubber handle is released by the user */
	FSimpleDelegate OnEndScrubberMovement;
	FOnRangeChanged OnViewRangeChanged;
	FOnFormatTickLabel OnFormatTickLabel;
	uint32 MinPixelsPerDisplayTick = 0u;

	/** Brush for drawing an upwards facing scrub handle */
	const FSlateBrush* ScrubHandleUp;
	/** Brush for drawing a downwards facing scrub handle */
	const FSlateBrush* ScrubHandleDown;
	/** Brush for drawing cursor background to visualize cursor size */
	const FSlateBrush* CursorBackground;
	/** Total mouse delta during dragging **/
	float DistanceDragged;
	/** If we are dragging the scrubber */
	bool bDraggingScrubber;
	/** If we are currently panning the panel */
	bool bPanning;
	/***/
	TSharedPtr<SScrollBar> Scrollbar;
	FVector2f SoftwareCursorPosition;
	FVector2f DesiredSize;

	
};

#undef UE_API
