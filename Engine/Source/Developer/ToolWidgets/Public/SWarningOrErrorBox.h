// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Layout/SBorder.h"

#define UE_API TOOLWIDGETS_API

enum class EMessageStyle
{
	Warning,
	Error
};

class SWarningOrErrorBox : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SWarningOrErrorBox)
		: _MessageStyle(EMessageStyle::Warning)
		, _Padding(16.0f)
		, _IconSize(24,24)
		, _AutoWrapText(true)
		, _ContentHAlign(EHorizontalAlignment::HAlign_Right)
		, _ContentVAlign(EVerticalAlignment::VAlign_Center)
		, _Content()
	{}
		SLATE_ATTRIBUTE(FText, Message)
		SLATE_ATTRIBUTE(EMessageStyle, MessageStyle)
		SLATE_ARGUMENT(FMargin, Padding)
		SLATE_ARGUMENT(FVector2D, IconSize)
		SLATE_ARGUMENT(bool, AutoWrapText)

		/** The Content horizontal alignment. */
		SLATE_ARGUMENT(EHorizontalAlignment, ContentHAlign)

		/** The Content vertical alignment. */
		SLATE_ARGUMENT(EVerticalAlignment, ContentVAlign)

		SLATE_DEFAULT_SLOT( FArguments, Content )

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

private:
	TAttribute<EMessageStyle> MessageStyle;
};

#undef UE_API
