// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WidgetDrawerConfig.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API TOOLWIDGETS_API

DECLARE_DELEGATE_OneParam(FOnStatusBarDrawerTargetHeightChanged, float);

class SWindow;
class SDockTab;
class SWidget;
class SDrawerOverlay;
class SHorizontalBox;

struct FOpenDrawerData
{
	FName DrawerId;
	TSharedPtr<SDrawerOverlay> DrawerOverlay;
	TWeakPtr<SWindow> WindowWithOverlayContent;

	bool IsValid() const
	{
		return !DrawerId.IsNone();
	}

	bool operator==(const FName InDrawerId) const
	{
		return DrawerId == InDrawerId;
	}
};

class SWidgetDrawer : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SWidgetDrawer)	
	{}
	SLATE_END_ARGS()
public:
	
	UE_API ~SWidgetDrawer();

	/**
	 * @param InArgs
	 * @param InStatusBarName Unique ID used to for writing to config file. Config file saves target drawer height. 
	 */
	UE_API void Construct(const FArguments& InArgs, FName InStatusBarName);
	
	virtual bool SupportsKeyboardFocus() const { return false; }
	

	/**
	 * Registers a new drawer with this status bar. Registering will add a button to open and close the drawer
	 */
	UE_API void RegisterDrawer(FWidgetDrawerConfig&& Drawer, int32 SlotIndex = INDEX_NONE);

	/**
	 * Unregisters drawer with the given DrawerId.
	 * This will dismiss the drawer immediately if its opened, and destroy the status bar widget that opens/closes the drawer
	 *
	 * @param DrawerId name of the registered drawer to unregister 
	 */
	UE_API void UnregisterDrawer(FName DrawerId);

	/**
	 * Opens a drawer
	 * 
	 * @param DrawerId Name of the registered drawer to open 
	 */
	UE_API void OpenDrawer(const FName DrawerId);

	/**
	 * Dismisses an open drawer with an animation.  The drawer contents are removed once the animation is complete
	 * 
	 * @param NewlyFocusedWidget Optional widget to focus 
	 * @return true if any open drawer was dismissed
	 */
	UE_API bool DismissDrawer(const TSharedPtr<SWidget>& NewlyFocusedWidget);

	/**
	 * Closes a drawer immediately with no closing animation. Needed for when UI is shutting down or to prevent conflicts where a drawer is open in two places at once. 
	 * @param DrawerId The name of the drawer to close. If no name is specified all drawers are closed immediately
	 */
	UE_API void CloseDrawerImmediately(FName DrawerId = NAME_None);

	/** Is a specific registered drawer currently open */
	UE_API bool IsDrawerOpened(const FName DrawerId) const;

	/** Is any drawer other than the one specified by DrawerId opened */
	UE_API bool IsAnyOtherDrawerOpened(const FName DrawerId) const;

	FName GetDrawerName() const { return DrawerName; }
	UE_API FString GetSerializableName() const;

private:
	
	/** Called when global focus changes which is used to determine if we should close an opened content browser drawer */
	UE_API void OnGlobalFocusChanging(const FFocusEvent& FocusEvent, const FWeakWidgetPath& OldFocusedWidgetPath, const TSharedPtr<SWidget>& OldFocusedWidget, const FWidgetPath& NewFocusedWidgetPath, const TSharedPtr<SWidget>& NewFocusedWidget);
	/** Called when active tab changes which is used to determine if we should close an opened content browser drawer */
	UE_API void OnActiveTabChanged(TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated);
	UE_API void OnDrawerHeightChanged(float TargetHeight);

	UE_API TSharedRef<SWidget> MakeStatusBarDrawerButton(const FWidgetDrawerConfig& Drawer);

	UE_API FReply OnDrawerButtonClicked(const FName DrawerId);
	
	UE_API void CloseDrawerImmediatelyInternal(const FOpenDrawerData& Data);
	
private:

	TArray<FWidgetDrawerConfig> RegisteredDrawers;

	TMap<FName, TWeakPtr<SWidget>> DrawerIdToContentWidget;

	TArray<FOpenDrawerData> DismissingDrawers;
	
	TSharedPtr<SHorizontalBox> DrawerBox;

	FName DrawerName;
	FOpenDrawerData OpenedDrawer;
};

#undef UE_API
