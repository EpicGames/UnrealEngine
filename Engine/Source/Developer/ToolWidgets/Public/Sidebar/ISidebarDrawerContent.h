// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"

class FName;
class FText;
class SWidget;

class ISidebarDrawerContent : public TSharedFromThis<ISidebarDrawerContent>
{
public:
	virtual ~ISidebarDrawerContent() = default;

	/** Gets the unique Id used to identify the sidebar drawer. */
	virtual FName GetUniqueId() const = 0;

	/** Gets the section unique Id used to identify the drawer section. */
	virtual FName GetSectionId() const = 0;

	/** Gets the section text to display on the section button. */
	virtual FText GetSectionDisplayText() const = 0;

	/** Returns true if this section should be displayed. */
	virtual bool ShouldShowSection() const { return true; }

	/** Sets the optional slot height stretch coefficient. If not set, auto sizes the slot. */
	virtual TOptional<float> GetSectionFillHeight() const { return 1.f; }

	/** Gets the section sort order. Lower values will be put at the above higher values. */
	virtual int32 GetSortOrder() const { return 0; }

	/** Constructs the content widget for a sidebar drawer. */
	virtual TSharedRef<SWidget> CreateContentWidget() = 0;

	/** Event that fires when the drawer is opened or closed. */
	virtual void OnDrawerOpenClose(const bool bOpen) {}
};
