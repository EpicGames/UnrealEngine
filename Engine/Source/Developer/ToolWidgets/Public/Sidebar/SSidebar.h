// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SidebarDrawerConfig.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API TOOLWIDGETS_API

class FExtender;
class FSidebarDrawer;
class ISidebarDrawerContent;
class SBox;
class SOverlay;
class SScrollBox;
class SSidebarButton;
class SSidebarContainer;
class SSidebarDrawer;
class STabDrawer;
class SVerticalBox;
class UToolMenu;
struct FSidebarState;
struct FTabId;

/**
 * The direction that a tab drawer opens relative to the location of the sidebar.
 * NOTE: Effort has been made to support top and bottom sidebar locations, but this has not been thoroughly tested
 *   and ironed out because there is currently no use case.
 */
enum class ESidebarTabLocation : uint8
{
	/** Open from left to right */
	Left,
	/** Open from right to left */
	Right,
	/** Open from bottom to top */
	Top,
	/** Open from top to bottom */
	Bottom
};

DECLARE_DELEGATE_OneParam(FOnSidebarStateChanged, const FSidebarState& /*InNewState*/);

/**
 * Static sidebar tab widget that cannot be dragged or moved to a different location. Multiple drawers can be registered
 * to a single sidebar and each drawer can have its own sections, each of which can be displayed single, in combination,
 * or all together through buttons at the top of the drawer.
 */
class SSidebar : public SCompoundWidget
{
public:
	friend class SSidebarContainer;

	static constexpr float MinTabButtonSize = 100.f;
	static constexpr float MaxTabButtonSize = 200.f;
	static constexpr float TabButtonThickness = 25.f;

	SLATE_BEGIN_ARGS(SSidebar)
		: _HideWhenAllDocked(false)
		, _AlwaysUseMaxButtonSize(false)
		, _DisablePin(false)
		, _DisableDock(false)
	{}
		/** The direction that a tab drawer opens relative to the location of the sidebar. */
		SLATE_ARGUMENT(ESidebarTabLocation, TabLocation)
		/** The initial size of a drawer for the sidebar. */
		SLATE_ARGUMENT(float, InitialDrawerSize)
		/** Delegate used to retrieve the main inner content of the sidebar that will be overlayed. */
		SLATE_ARGUMENT(FOnGetContent, OnGetContent)
		/** Hides the sidebar when all drawers is docked. NOTE: Must provide a way to manually undock the drawer to restore the sidebar visibility. */
		SLATE_ARGUMENT(bool, HideWhenAllDocked)
		/** Forces the sidebar tab buttons to always be a uniform size of max. */
		SLATE_ARGUMENT(bool, AlwaysUseMaxButtonSize)
		/** Disables the ability to pin a drawer. */
		SLATE_ARGUMENT(bool, DisablePin)
		/** Disables the ability to dock a drawer. */
		SLATE_ARGUMENT(bool, DisableDock)
		/** Event triggered when the sidebar state changes. */
		SLATE_EVENT(FOnSidebarStateChanged, OnStateChanged)
	SLATE_END_ARGS()

	UE_API virtual ~SSidebar() override;

	/**
	 * Constructs the sidebar widget.
	 * 
	 * @param InArgs Widget construction arguments
	 * @param InContainerWidget Container widget used to manage the drawer sidebar widgets
	 * @param InDockLocation Parent widget that will contain the drawer content widget when docked
	 */
	UE_API void Construct(const FArguments& InArgs, const TSharedRef<SSidebarContainer>& InContainerWidget);

	/**
	 * Registers and displays a new drawer in the sidebar.
	 * 
	 * @param InDrawerConfig Configuration info for the new drawer
	 * 
	 * @return True if the new drawer registration was successful.
	 */
	UE_API bool RegisterDrawer(FSidebarDrawerConfig&& InDrawerConfig);

	/**
	 * Unregisters and removes a drawer from the sidebar.
	 *
	 * @param InDrawerId Unique drawer Id to unregister
	 * 
	 * @return True if the drawer removal was successful.
	 */
	UE_API bool UnregisterDrawer(const FName InDrawerId);

	/**
	 * Checks if a drawer exists in the sidebar.
	 * 
	 * @param InDrawerId Unique drawer Id to check exists
	 * 
	 * @return True if the sidebar contains a drawer with the specified Id.
	 */
	UE_API bool ContainsDrawer(const FName InDrawerId) const;

	/** @return The number of drawers that exist in the sidebar. */
	UE_API int32 GetDrawerCount() const;

	/**
	 * Registers and displays a new drawer section in the sidebar.
	 * 
	 * @param InDrawerId Unique drawer Id to register
	 * @param InSection Drawer content interface for the section
	 * 
	 * @return True if the new drawer section registration was successful.
	 */
	UE_API bool RegisterDrawerSection(const FName InDrawerId, const TSharedPtr<ISidebarDrawerContent>& InSection);

	/**
	 * Unregisters and removes a drawer section from the sidebar.
	 * 
	 * @param InDrawerId Unique drawer Id that contains the section to unregister
	 * @param InSectionId Unique drawer section Id to unregister
	 * 
	 * @return True if the drawer removal was successful.
	 */
	UE_API bool UnregisterDrawerSection(const FName InDrawerId, const FName InSectionId);

	/**
	 * Checks if a drawer section exists within a sidebar drawer.
	 * 
	 * @param InDrawerId Unique Id of the drawer to look for the section in
	 * @param InSectionId Unique drawer section Id to check exists
	 * 
	 * @return True if the sidebar contains a drawer with the specified Id.
	 */
	UE_API bool ContainsDrawerSection(const FName InDrawerId, const FName InSectionId) const;

	/**
	 * Attempt to open a specific drawer in the sidebar.
	 *
	 * @param InDrawerId Unique Id of the drawer to attempt to open
	 * 
	 * @return True if the drawer exists in this sidebar and was opened.
	 */
	UE_API bool TryOpenDrawer(const FName InDrawerId);

	/** Closes any drawers that are open. */
	UE_API void CloseAllDrawers(const bool bInAnimate = true);

	/** @return True if the sidebar has any drawer that is opened. */
	UE_API bool HasDrawerOpened() const;

	/**
	 * Checks if a drawer is opened.
	 * 
	 * @param InDrawerId Unique Id of the drawer to check
	 * 
	 * @return True if the specified drawer is currently opened.
	 */
	UE_API bool IsDrawerOpened(const FName InDrawerId) const;

	/** @return The unique drawer Id that is currently open. */
	UE_API FName GetOpenedDrawerId() const;

	/** @return True if the sidebar has any drawer that is pinned. */
	UE_API bool HasDrawerPinned() const;

	/**
	 * Checks if a drawer is pinned.
	 * 
	 * @param InDrawerId Unique Id of the drawer to check
	 * 
	 * @return True if the specified drawer is currently pinned.
	 */
	UE_API bool IsDrawerPinned(const FName InDrawerId) const;

	/** @return List of drawer Ids that are pinned. */
	UE_API TSet<FName> GetPinnedDrawerIds() const;

	/**
	 * Pins a drawer so it stays open even when focus is lost.
	 * 
	 * @param InDrawerId Unique Id of the drawer to pin
	 * @param bInIsPinned New pin state to set
	 */
	UE_API void SetDrawerPinned(const FName InDrawerId, const bool bInIsPinned);

	/** @return True if the sidebar has any drawer that is docked. */
	UE_API bool HasDrawerDocked() const;

	/** @return True if the specified drawer Id is docked. */
	UE_API bool IsDrawerDocked(const FName InDrawerId) const;

	/** @return List of drawer Ids that are docked. */
	UE_API TSet<FName> GetDockedDrawerIds() const;

	/**
	 * Docks a drawer so it embeds itself into the content.
	 * 
	 * @param InDrawerId Unique Id of the drawer to dock
	 * @param bInIsDocked New dock state to set
	 */
	UE_API void SetDrawerDocked(const FName InDrawerId, const bool bInIsDocked);

	/** Undocks any drawers that are docked. */
	UE_API void UndockAllDrawers();

	/** Unpins any drawers that are pinned. */
	UE_API void UnpinAllDrawers();

	/** @return True if the sidebar is set to animate horizontally. */
	UE_API bool IsHorizontal() const;

	/** @return True if the sidebar is set to animate vertically. */
	UE_API bool IsVertical() const;

	/** @return The current state of the sidebar to save/restore. */
	UE_API FSidebarState GetState() const;

	/** @return The location of the sidebar (left, right, top, bottom). */
	UE_API ESidebarTabLocation GetTabLocation() const;

	UE_API TSharedRef<SWidget> GetMainContent() const;

	/** Rebuilds a drawer's content (unless it has been overriden). */
	UE_API void RebuildDrawer(const FName InDrawerId, const bool bInOnlyIfOpen = true);

private:
	EVisibility GetSidebarVisibility() const;

	void RecreateDrawerWidgets();

	UE_API void OnTabDrawerButtonPressed(const TSharedRef<FSidebarDrawer>& InDrawer);
	UE_API void OnDrawerTabPinToggled(const TSharedRef<FSidebarDrawer>& InDrawer, const bool bIsPinned);
	UE_API void OnDrawerTabDockToggled(const TSharedRef<FSidebarDrawer>& InDrawer, const bool bIsDocked);

	UE_API TSharedRef<SWidget> OnGetTabDrawerContextMenuWidget(TSharedRef<FSidebarDrawer> InDrawer);
	UE_API void BuildOptionsMenu(UToolMenu* const InMenu);

	/** Removes a single drawer for a specified tab from this sidebar. Removal is done instantly not waiting for any close animation. */
	UE_API void RemoveDrawer(const TSharedRef<FSidebarDrawer>& InDrawer);

	/** Removes all drawers instantly (including drawers for pinned tabs). */
	UE_API void RemoveAllDrawers();

	UE_API TSharedPtr<FSidebarDrawer> FindDrawer(const FName InDrawerId) const;

	UE_API void SetWidgetDrawerSize(const TSharedRef<FSidebarDrawer>& InDrawer);

	UE_API bool AreAllDrawersDocked() const;

	UE_API const TArray<TSharedRef<FSidebarDrawer>>& GetAllDrawers() const;

	TWeakPtr<SSidebarContainer> ContainerWidgetWeak;

	ESidebarTabLocation TabLocation = ESidebarTabLocation::Right;
	FOnGetContent OnGetContent;
	bool bHideWhenAllDocked = false;
	bool bAlwaysUseMaxButtonSize = false;
	bool bDisablePin = false;
	bool bDisableDock = false;
	FOnSidebarStateChanged OnStateChanged;

	TSharedPtr<SScrollBox> TabButtonContainer;

	TArray<TSharedRef<FSidebarDrawer>> Drawers;
};

#undef UE_API
