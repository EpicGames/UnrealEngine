// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sidebar/SidebarState.h"
#include "Sidebar/SSidebar.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API TOOLWIDGETS_API

class SSidebar;
class SSidebarDrawer;

/**
 * A container for a sidebar widget that manages the slider drawer overlay widgets
 * and a default docking location for all drawers.
 */
class SSidebarContainer : public SCompoundWidget
{
public:
	friend class SSidebar;

	SLATE_BEGIN_ARGS(SSidebarContainer)
	{}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	UE_API void RebuildSidebar(const TSharedRef<SSidebar>& InSidebarWidget, const FSidebarState& InState);

	UE_API float GetContentSlotSize() const;
	UE_API float GetSidebarSlotSize() const;

	UE_API EOrientation GetSplitterOrientation() const;
	UE_API ESidebarTabLocation GetTabLocation() const;
	UE_API float GetCurrentDrawerSize() const;
	UE_API UE::Slate::FDeprecateVector2DResult GetOverlaySize() const;

	UE_API void CloseAllDrawerWidgets(const bool bInAnimate);

	/** Reopens the pinned tab only if there are no other open drawers. This should be used to bring pinned tabs back after other tabs lose focus/are closed. */
	UE_API void SummonPinnedTabIfNothingOpened();

	/** Updates the appearance of drawer tabs. */
	UE_API void UpdateDrawerTabAppearance();

	UE_API FName GetOpenedDrawerId() const;

protected:
	UE_API void Reconstruct(const FSidebarState& InState = FSidebarState());

	UE_API TSharedRef<SWidget> ConstructBoxPanel(const FSidebarState& InState);
	UE_API void ConstructSplitterPanel(const FSidebarState& InState);

	UE_API FMargin CalculateSlotMargin() const;

	UE_API void CreateDrawerWidget(const TSharedRef<FSidebarDrawer>& InDrawer);

	UE_API TSharedRef<SWidget> GetSidebarDrawerContent(const TSharedRef<FSidebarDrawer>& InDrawer) const;

	UE_API bool AddDrawerOverlaySlot(const TSharedRef<FSidebarDrawer>& InDrawer);
	UE_API bool RemoveDrawerOverlaySlot(const TSharedRef<FSidebarDrawer>& InDrawer, const bool bInAnimate);

	UE_API void AddContentDockSlot();
	UE_API void RemoveContentDockSlot();

	UE_API void AddSidebarDockSlot(const FName InDockDrawerId);
	UE_API void RemoveSidebarDockSlot();

	UE_API void OnTabDrawerFocusLost(const TSharedRef<SSidebarDrawer>& InDrawerWidget);
	UE_API void OnOpenAnimationFinish(const TSharedRef<SSidebarDrawer>& InDrawerWidget);
	UE_API void OnCloseAnimationFinish(const TSharedRef<SSidebarDrawer>& InDrawerWidget);
	UE_API void OnDrawerSizeChanged(const TSharedRef<SSidebarDrawer>& InDrawerWidget, const float InNewPixelSize);

	UE_API EActiveTimerReturnType OnOpenPendingDrawerTimer(const double InCurrentTime, const float InDeltaTime);

	UE_API void OpenDrawerNextFrame(const TSharedRef<FSidebarDrawer>& InDrawer, const bool bInAnimate = true);
	UE_API void OpenDrawer_Internal(const TSharedRef<FSidebarDrawer>& InDrawer, const bool bInAnimate = true);
	UE_API void CloseDrawer_Internal(const TSharedRef<FSidebarDrawer>& InDrawer, const bool bInAnimate = true
		, const bool bInSummonPinnedTabIfNothingOpened = true);

	UE_API void DockDrawer_Internal(const TSharedRef<FSidebarDrawer>& InDrawer);
	UE_API void UndockDrawer_Internal(const TSharedRef<FSidebarDrawer>& InDrawer);

	UE_API TSharedPtr<FSidebarDrawer> FindDrawer(const FName InDrawerId) const;
	UE_API TSharedPtr<FSidebarDrawer> FindFirstPinnedTab() const;

	UE_API TSharedPtr<SSidebarDrawer> FindOpenDrawerWidget(const TSharedRef<FSidebarDrawer>& InDrawer) const;

	UE_API TSharedPtr<FSidebarDrawer> GetForegroundDrawer() const;

	/** Event that occurs when the docked main content slot is being resized by the user (while mouse down). */
	UE_API void OnContentSlotResizing(const float InFillPercent);
	/** Event that occurs when the docked sidebar slot is being resized by the user (while mouse down). */
	UE_API void OnSidebarSlotResizing(const float InFillPercent);

	/** Event that occurs when the main splitter widget has finished being resized by the user (mouse up). */
	UE_API void OnSplitterResized();

	UE_API int32 GetContentSlotIndex() const;
	UE_API int32 GetSidebarSlotIndex() const;

	void BroadcastDrawerOpenClose(const TSharedRef<FSidebarDrawer>& InDrawer, const bool bOpen);

	/** The sidebar widget associated with this container. One sidebar widget per container. */
	TSharedPtr<SSidebar> SidebarWidget;

	/** The main splitter widget used when a drawer is docked. */
	TSharedPtr<SSplitter> MainSplitter;

	/** Overlay used to draw drawer widgets on top of the rest of the content. */
	TSharedPtr<SOverlay> DrawersOverlay;

	/** Generally speaking one drawer is only ever open at once but we animate any previous drawer
	 * closing so there could be more than one while an animation is playing. A docked drawer is
	 * also considered open, along with any user opened/pinned drawers. */
	TArray<TSharedRef<SSidebarDrawer>> OpenDrawerWidgets;

	TArray<TSharedRef<SSidebarDrawer>> ClosingDrawerWidgets;

	TWeakPtr<FSidebarDrawer> PendingTabToOpen;
	bool bAnimatePendingTabOpen = false;
	TSharedPtr<FActiveTimerHandle> OpenPendingDrawerTimerHandle;

	float ContentSizePercent = 0.8f;
	float SidebarSizePercent = 0.2f;

	TAttribute<float> ContentSlotSize;
	TAttribute<float> SidebarSlotSize;

	bool bWantsToAutoDock = false;
	float ContentSizeBeforeResize = 0.f;
	float SidebarSizeBeforeResize = 0.f;
};

#undef UE_API
