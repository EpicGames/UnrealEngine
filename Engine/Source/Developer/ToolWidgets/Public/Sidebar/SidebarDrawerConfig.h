// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "SidebarState.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class SWidget;
struct FSlateBrush;

/** Configuration information used to register a sidebar drawer. */
struct FSidebarDrawerConfig
{
	bool operator==(const FName InOtherId) const
	{
		return UniqueId == InOtherId;
	}

	bool operator!=(const FName InOtherId) const
	{
		return UniqueId != InOtherId;
	}

	bool operator==(const FSidebarDrawerConfig& InOther) const
	{
		return UniqueId == InOther.UniqueId;
	}

	bool operator!=(const FSidebarDrawerConfig& InOther) const
	{
		return UniqueId != InOther.UniqueId;
	}

	/** Unique Id used to identify the drawer */
	FName UniqueId;

	/** Primary sort order of the drawer tab button. Secondary sort by ButtonText property. */
	float SortOrder = 10.f;

	/** Text to display on the drawer tab button */
	TAttribute<FText> ButtonText;

	/** ToolTip text to display for the drawer tab button */
	TAttribute<FText> ToolTipText;

	/** Icon to display for the drawer tab button */
	TAttribute<const FSlateBrush*> Icon;

	/** The state of the sidebar drawer. (Visible, Pinned, Docked, Size) */
	FSidebarDrawerState InitialState;

	/** Optional content widget to use instead of registering sections. */
	TSharedPtr<SWidget> OverrideContentWidget;

	// @TODO: events for pin state, dock state, section state changes?
};
