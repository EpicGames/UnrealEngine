// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SidebarState.generated.h"

#define UE_API TOOLWIDGETS_API

/** Represents the state of a sidebar drawer to be saved/restored to/from config. */
USTRUCT()
struct FSidebarDrawerState
{
	GENERATED_BODY()

	FSidebarDrawerState() {}
	FSidebarDrawerState(const FName InDrawerId)
	{
		DrawerId = InDrawerId;
	}

	UPROPERTY()
	FName DrawerId = NAME_None;

	/** Names of all sections that were last selected */
	UPROPERTY()
	TSet<FName> SelectedSections;

	UPROPERTY()
	bool bIsPinned = false;

	UPROPERTY()
	bool bIsDocked = false;
};

/** Represents the state of a sidebar to be saved/restored to/from config. */
USTRUCT()
struct FSidebarState
{
	GENERATED_BODY()

public:
	static constexpr float DefaultSize = 0.25f;
	static constexpr float MinSize = 0.005f;
	static constexpr float MaxSize = 0.5f;
	static constexpr float AutoDockThresholdSize = 0.05f;

	/** @return True if any property has been changed from default */
	UE_API bool IsValid() const;

	UE_API bool IsHidden() const;
	UE_API bool IsVisible() const;

	UE_API void SetHidden(const bool bInHidden);
	UE_API void SetVisible(const bool bInVisible);

	UE_API float GetDrawerSize() const;
	UE_API void SetDrawerSize(const float InSize);

	UE_API void GetDrawerSizes(float& OutDrawerSize, float& OutContentSize) const;
	UE_API void SetDrawerSizes(const float InDrawerSize, const float InContentSize);

	UE_API const TArray<FSidebarDrawerState>& GetDrawerStates() const;

	UE_API FSidebarDrawerState& FindOrAddDrawerState(const FSidebarDrawerState& InDrawerState);
	UE_API const FSidebarDrawerState* FindDrawerState(const FSidebarDrawerState& InDrawerState);

	/** Saves the state of a drawer. If the drawers state already exists in config, it will be replaced. */
	UE_API void SaveDrawerState(const FSidebarDrawerState& InState);

protected:
	UPROPERTY()
	bool bHidden = false;

	UPROPERTY()
	float DrawerSize = 0.f;

	/** Save the other splitter slot size to exactly restore the size when a drawer is docked in a SSplitter widget. */
	UPROPERTY()
	float ContentSize = 0.f;

	UPROPERTY()
	TArray<FSidebarDrawerState> DrawerStates;
};

#undef UE_API
