// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyle.h"
#include "UObject/Object.h"

#include "ToolWidgetsSlateTypes.generated.h"

enum class EActionButtonType : uint8
{
	Default,		// A default, grey button
	Simple,			// A button with a left-aligned label, and no background or border when unhovered
	Primary,		// A primary blue button with white text, use for buttons when it's the next logical step (ie. "Next", "OK")
	Positive,		// A grey button with a green icon indicating a positive action, like "Add"
	Warning,		// A grey button with a yellow icon indicating a negative action, like "Mute"
	Error,			// A grey button with a red icon indicating a negative action, like "Cancel"

	Num,
};

/**
 * Represents the appearance of an SActionButton
 */
USTRUCT(BlueprintType)
struct FActionButtonStyle : public FSlateWidgetStyle
{
	GENERATED_BODY()

	TOOLWIDGETS_API FActionButtonStyle();

	TOOLWIDGETS_API virtual void GetResources(TArray<const FSlateBrush*>& OutBrushes) const override;

	static TOOLWIDGETS_API const FName TypeName;
	virtual const FName GetTypeName() const override;;

	static TOOLWIDGETS_API const FActionButtonStyle& GetDefault();

	/**
	 * The type to use for our SActionButton.
	 */
	EActionButtonType GetActionButtonType() const;
	FActionButtonStyle& SetActionButtonType(const EActionButtonType InActionButtonType);

	/**
	 * The style to use for our SButton.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	FButtonStyle ButtonStyle;
	FActionButtonStyle& SetButtonStyle(const FButtonStyle& InButtonStyle);

	/**
	 * The style to use for our SButton when an icon is present. ButtonStyle used if not specified.
	 */
	const FButtonStyle& GetIconButtonStyle() const;
	FActionButtonStyle& SetIconButtonStyle(const FButtonStyle& InButtonStyle);

	/**
	 * Spacing between button's border and the content. Default uses ButtonStyle.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	TOptional<FMargin> ButtonContentPadding;

	FMargin GetButtonContentPadding() const;
	FActionButtonStyle& SetButtonContentPadding(const FMargin& InContentPadding);

	/**
	 * The style to use for our SComboButton.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	FComboButtonStyle ComboButtonStyle;
	FActionButtonStyle& SetComboButtonStyle(const FComboButtonStyle& InComboButtonStyle);

	/**
	 * Whether to show a down arrow for the combo button
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	bool bHasDownArrow;
	FActionButtonStyle& SetHasDownArrow(const bool& bInHasDownArrow);

	/**
	 * Spacing between button's border and the content. Default uses ComboButtonStyle.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	TOptional<FMargin> ComboButtonContentPadding;

	FMargin GetComboButtonContentPadding() const;
	FActionButtonStyle& SetComboButtonContentPadding(const FMargin& InContentPadding);

	/**
	 * Horizontal Content alignment within the button.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	TEnumAsByte<EHorizontalAlignment> HorizontalContentAlignment;
	FActionButtonStyle& SetHorizontalContentAlignment(const EHorizontalAlignment& InAlignment);

	/**
	 * The style to use for the button Text.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	FTextBlockStyle TextBlockStyle;
	FActionButtonStyle& SetTextBlockStyle(const FTextBlockStyle& InTextBlockStyle);

	/**
	 * Icon Brush to use.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	TOptional<FSlateBrush> IconBrush;
	FActionButtonStyle& SetIconBrush(const FSlateBrush& InIconBrush);

	/**
	 * Icon Color/Tint, defaults is determined by ActionButtonType.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	TOptional<FSlateColor> IconColorAndOpacity;
	FActionButtonStyle& SetIconColorAndOpacity(const FSlateColor& InIconColorAndOpacity);

	/**
	 * If set and the button's icon is non-null, overrides the button style's additional spacing between the button's border and the content when not pressed.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	TOptional<FMargin> IconNormalPadding;
	FActionButtonStyle& SetIconNormalPadding(const FMargin& InIconNormalPadding);

	/**
	 * If set and the button's icon is non-null, overrides the button style's additional spacing between the button's border and the content when pressed.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	TOptional<FMargin> IconPressedPadding;
	FActionButtonStyle& SetIconPressedPadding(const FMargin& InIconPressedPadding);

	/**
	* Unlinks all colors in this style.
	* @see FSlateColor::Unlink
	 */
	void UnlinkColors();

private:
	/**
	 * The type to use for our SActionButton.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance", meta = (AllowPrivateAccess = "true", GetOptions = "Engine.ToolSlateWidgetTypesFunctionLibrary.GetActionButtonTypeNames"))
	FName ActionButtonType;

	/**
	 * The style to use for our SButton when an icon is present. ButtonStyle used if not specified.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance", meta = (AllowPrivateAccess = "true"))
	TOptional<FButtonStyle> IconButtonStyle;
};

UCLASS(MinimalAPI)
class UToolSlateWidgetTypesFunctionLibrary
	: public UObject
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	UFUNCTION()
	static const TArray<FName>& GetActionButtonTypeNames();
#endif
};
