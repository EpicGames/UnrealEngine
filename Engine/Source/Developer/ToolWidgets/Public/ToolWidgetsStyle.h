// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

namespace UE::ToolWidgets
{
	class FToolWidgetsStyle final
		: public FSlateStyleSet
	{
	public:
		static FName StyleName;

		static TOOLWIDGETS_API FToolWidgetsStyle& Get();

	private:
		FToolWidgetsStyle();
		virtual ~FToolWidgetsStyle() override;
	};
}
