// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SBoxPanel.h"

class SWidget;

DECLARE_DELEGATE_OneParam(FOnStatusBarDrawerOpened, FName StatusBarName)
DECLARE_DELEGATE_OneParam(FOnStatusBarDrawerDismissed, const TSharedPtr<SWidget>&)
DECLARE_DELEGATE_RetVal_OneParam(EVisibility, FOnGetVisiblity, FName StatusBarName)

struct FWidgetDrawerConfig
{
	FWidgetDrawerConfig(FName InUniqueId)
		: UniqueId(InUniqueId)
	{}

	FName UniqueId;
	FOnGetContent GetDrawerContentDelegate;
	FOnGetVisiblity GetDrawerButtonVisibilityDelegate;
	FOnStatusBarDrawerOpened OnDrawerOpenedDelegate;
	FOnStatusBarDrawerDismissed OnDrawerDismissedDelegate;

	TSharedPtr<SWidget> CustomWidget;
	FText ButtonText;
	FText ToolTipText;
	const FSlateBrush* Icon = nullptr;
	TSharedRef<SHorizontalBox> CustomButtonWidgets = SNew(SHorizontalBox);

	bool operator==(const FName& OtherId) const
	{
		return UniqueId == OtherId;
	}

	bool operator==(const FWidgetDrawerConfig& Other) const
	{
		return UniqueId == Other.UniqueId;
	}
};