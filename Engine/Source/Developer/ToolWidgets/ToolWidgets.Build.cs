// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ToolWidgets : ModuleRules
{
	public ToolWidgets(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		/** NOTE: THIS MODULE SHOULD NOT EVER DEPEND ON UNREALED. 
		 * If you are adding a reusable widget that depends on UnrealEd, add it to EditorWidgets instead
		 */
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Slate",
				"SlateCore",
				"InputCore",
				"ToolMenus",
				"CoreUObject",
				"TypedElementFramework"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AppFramework",
				"ApplicationCore"
			}
		);

		bDisableAutoRTFMInstrumentation = true;
	}
}
